{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Chaos Outfit Asset",
	"Description": "Outfit Asset plugin to create and assemble outfits made of Cloth Assets.",
	"Category": "Physics",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "ChaosOutfitAssetDataflowNodes",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ChaosOutfitAssetEngine",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ChaosOutfitAssetEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64", "Mac", "Linux" ]
		}
	],
	"Plugins": [
		{
			"Name": "ChaosClothAsset",
			"Enabled": true
		},
		{
			"Name": "Dataflow",
			"Enabled": true
		},
		{
			"Name": "MeshResizing",
			"Enabled": true
		},
		{
			"Name": "ChaosClothAssetEditorCore",
			"Enabled": true,
			"TargetAllowList" : [ "Editor" ]
		}
	]
}
