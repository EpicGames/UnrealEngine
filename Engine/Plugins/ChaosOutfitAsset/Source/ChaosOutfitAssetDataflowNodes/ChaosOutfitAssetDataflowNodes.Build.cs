// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ChaosOutfitAssetDataflowNodes : ModuleRules
{
	public ChaosOutfitAssetDataflowNodes(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetRegistry",
				"Core",
				"CoreUObject",
				"Chaos",
				"ChaosClothAsset",
				"ChaosClothAssetEngine",
				"ChaosOutfitAssetEngine",
				"DataflowCore",
				"Engine",
				"MeshResizingCore",
			}
		);
	}
}
