// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosOutfitAsset/ClothAssetAnyType.h"
#include "ChaosOutfitAsset/FilterSizedOutfitNode.h"
#include "ChaosOutfitAsset/GetClothAssetNode.h"
#include "ChaosOutfitAsset/GetOrMakeOutfitFromAssetNode.h"
#include "ChaosOutfitAsset/GetOutfitAssetNode.h"
#include "ChaosOutfitAsset/GetOutfitBodyPartsNode.h"
#include "ChaosOutfitAsset/GetOutfitClothCollectionsNode.h"
#include "ChaosOutfitAsset/GetOutfitRBFInterpolationDataNode.h"
#include "ChaosOutfitAsset/MakeOutfitNode.h"
#include "ChaosOutfitAsset/MakeSizedOutfitNode.h"
#include "ChaosOutfitAsset/MergeOutfitsNode.h"
#include "ChaosOutfitAsset/OutfitAssetTerminalNode.h"
#include "ChaosOutfitAsset/OutfitQueryNode.h"
#include "ChaosOutfitAsset/SetOutfitClothCollectionNode.h"
#include "ChaosOutfitAsset/SizedOutfitSourceAnyType.h"
#include "Dataflow/DataflowAnyTypeRegistry.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowNodeColorsRegistry.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "OutfitAssetDataflowNodesModule"

namespace UE::Chaos::OutfitAsset
{
	namespace Private
	{
		static const FLinearColor OutfitAssetNodeHeaderColor = FColor(162, 108, 99);
		static const FLinearColor OutfitAssetNodeBodyColor = FColor(18, 12, 11, 127);

		static void RegisterDataflowNodes()
		{
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("Outfit", OutfitAssetNodeHeaderColor, OutfitAssetNodeBodyColor);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosExtractBodyPartsArrayFromBodySizePartsNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosGetClothAssetNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosGetOutfitAssetNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosGetOutfitBodyPartsNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosGetOutfitClothCollectionsNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosGetOutfitRBFInterpolationDataNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosOutfitAssetFilterSizedOutfitNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosOutfitAssetGetOrMakeOutfitFromAssetNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosOutfitAssetMakeOutfitNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosOutfitAssetMakeSizedOutfitNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosOutfitAssetMergeOutfitsNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosSetOutfitClothCollectionNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosOutfitAssetTerminalNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosOutfitAssetOutfitQueryNode);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			// Move any deprecated node registrations under here
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

		static void RegisterAnyTypes()
		{
			constexpr float DefaultWireThickness = 1.5f;
			// Register Anytype for UChaosClothAssetBase (UChaosClothAsset and UChaosOutfitAsset)
			UE_DATAFLOW_REGISTER_ANYTYPE(FChaosClothAssetAnyType);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FChaosClothAssetAnyType", OutfitAssetNodeHeaderColor, DefaultWireThickness);
			UE_DATAFLOW_REGISTER_ANYTYPE(FChaosClothAssetArrayAnyType);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FChaosClothAssetArrayAnyType", OutfitAssetNodeHeaderColor, DefaultWireThickness);
			UE_DATAFLOW_REGISTER_ANYTYPE(FChaosClothAssetOrArrayAnyType);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FChaosClothAssetOrArrayAnyType", OutfitAssetNodeHeaderColor, DefaultWireThickness);
			UE_DATAFLOW_REGISTER_ANYTYPE(FChaosSizedOutfitSourceOrArrayAnyType);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FChaosSizedOutfitSourceOrArrayAnyType", OutfitAssetNodeHeaderColor, DefaultWireThickness);
		}
	}  // namespace Private

	class FOutfitAssetDataflowNodesModule : public IModuleInterface
	{
	public:
		virtual void StartupModule() override
		{
			Private::RegisterAnyTypes();
			Private::RegisterDataflowNodes();
		}

		virtual void ShutdownModule() override
		{
		}
	};
}  // namespace UE::Chaos::ClothAsset

IMPLEMENT_MODULE(UE::Chaos::OutfitAsset::FOutfitAssetDataflowNodesModule, ChaosOutfitAssetDataflowNodes);

#undef LOCTEXT_NAMESPACE
