// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowAnyType.h"
#include "Dataflow/DataflowTypePolicy.h"
#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosOutfitAsset/OutfitAsset.h"
#include "Misc/TVariant.h"

#include "ClothAssetAnyType.generated.h"

USTRUCT()
struct FChaosClothAssetOrArrayType
{
	GENERATED_BODY()

	TVariant<TObjectPtr<UChaosClothAssetBase>, TArray<TObjectPtr<UChaosClothAssetBase>>> AssetOrArray;

	FChaosClothAssetOrArrayType() = default;

	template <typename T UE_REQUIRES(TIsDerivedFrom<T, UChaosClothAssetBase>::Value)>
	FChaosClothAssetOrArrayType(const TObjectPtr<T>& Asset)
		: AssetOrArray(TInPlaceType<TObjectPtr<UChaosClothAssetBase>>(), Asset)
	{}

	template <typename T UE_REQUIRES(TIsDerivedFrom<T, UChaosClothAssetBase>::Value)>
	FChaosClothAssetOrArrayType(const TArray<TObjectPtr<T>>& Array)
		: AssetOrArray(TInPlaceType<TArray<TObjectPtr<UChaosClothAssetBase>>>(), Array)
	{}

	bool IsArray() const
	{
		return AssetOrArray.IsType<TArray<TObjectPtr<UChaosClothAssetBase>>>();
	}

	const TObjectPtr<UChaosClothAssetBase>& Get() const
	{
		check(!IsArray());
		return AssetOrArray.Get<TObjectPtr<UChaosClothAssetBase>>();
	}

	const TArray<TObjectPtr<UChaosClothAssetBase>>& GetArray() const
	{
		check(IsArray());
		return AssetOrArray.Get<TArray<TObjectPtr<UChaosClothAssetBase>>>();
	}

	bool Serialize(FArchive& Ar)
	{
		Ar << AssetOrArray;
		return true;
	}
};
template<>
struct TStructOpsTypeTraits<FChaosClothAssetOrArrayType> : public TStructOpsTypeTraitsBase2<FChaosClothAssetOrArrayType>
{
	enum
	{
		WithSerializer = true,
	};
};

UE_DATAFLOW_POLICY_DECLARE_TYPENAME(TObjectPtr<UChaosClothAssetBase>)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(TObjectPtr<UChaosClothAsset>)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(TObjectPtr<UChaosOutfitAsset>)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FChaosClothAssetOrArrayType)

/** UChaosClothAssetBase types. */
USTRUCT()
struct FChaosClothAssetAnyType : public FDataflowAnyType
{
	GENERATED_BODY()

	struct FTypePolicy :
		public TDataflowMultiTypePolicy<
		TObjectPtr<UChaosClothAssetBase>,
		TObjectPtr<UChaosClothAsset>,
		TObjectPtr<UChaosOutfitAsset>>
	{};

	using FPolicyType = FTypePolicy;
	using FStorageType = TObjectPtr<UChaosClothAssetBase>;

	UPROPERTY(EditAnywhere, Category = Value)
	TObjectPtr<UChaosClothAssetBase> Value = nullptr;
};

/** UChaosClothAssetBase array types. */
USTRUCT()
struct FChaosClothAssetArrayAnyType : public FDataflowAnyType
{
	GENERATED_BODY()

	struct FArrayPolicy :
		public TDataflowMultiTypePolicy<
			TArray<TObjectPtr<UChaosClothAssetBase>>,
			TArray<TObjectPtr<UChaosClothAsset>>,
			TArray<TObjectPtr<UChaosOutfitAsset>>>
	{};

	using FPolicyType = FArrayPolicy;
	using FStorageType = TArray<TObjectPtr<UChaosClothAssetBase>>;

	UPROPERTY(EditAnywhere, Category = Value)
	TArray<TObjectPtr<UChaosClothAssetBase>> Value;
};

/** UChaosClothAssetBase or array types. */
USTRUCT()
struct FChaosClothAssetOrArrayAnyType : public FDataflowAnyType
{
	GENERATED_BODY()

	struct FAssetOrArrayPolicy :
		public TDataflowMultiTypePolicy<
			TObjectPtr<UChaosClothAssetBase>,
			TObjectPtr<UChaosClothAsset>,
			TObjectPtr<UChaosOutfitAsset>,
			TArray<TObjectPtr<UChaosClothAssetBase>>,
			TArray<TObjectPtr<UChaosClothAsset>>,
			TArray<TObjectPtr<UChaosOutfitAsset>>>
	{};

	using FPolicyType = FAssetOrArrayPolicy;
	using FStorageType = FChaosClothAssetOrArrayType;

	UPROPERTY(EditAnywhere, Category = Value)
	FChaosClothAssetOrArrayType Value;
};
