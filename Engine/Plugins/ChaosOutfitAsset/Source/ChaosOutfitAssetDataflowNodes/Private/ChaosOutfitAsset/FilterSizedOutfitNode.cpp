// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosOutfitAsset/FilterSizedOutfitNode.h"
#include "ChaosOutfitAsset/Outfit.h"
#include "ChaosOutfitAsset/CollectionOutfitFacade.h"
#include "Engine/SkeletalMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FilterSizedOutfitNode)

#define LOCTEXT_NAMESPACE "ChaosOutfitAssetFilterSizedOutfitNode"

FChaosOutfitAssetFilterSizedOutfitNode::FChaosOutfitAssetFilterSizedOutfitNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Outfit);
	RegisterInputConnection(&SizeName);
	RegisterInputConnection(&TargetBody);
	RegisterOutputConnection(&Outfit, &Outfit);
	RegisterOutputConnection(&SizeName, &SizeName);
	RegisterOutputConnection(&OutfitCollection);
}

void FChaosOutfitAssetFilterSizedOutfitNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Chaos::OutfitAsset;

	if (Out->IsA(&Outfit) || Out->IsA(&SizeName) || Out->IsA(&OutfitCollection))
	{
		if (const TObjectPtr<const UChaosOutfit>& InOutfit = GetValue(Context, &Outfit))
		{
			FCollectionOutfitConstFacade InOutfitFacade(InOutfit->GetOutfitCollection());
			if (InOutfitFacade.IsValid())
			{
				FString InSizeName = GetValue(Context, &SizeName);
				if (!InSizeName.IsEmpty() && !InOutfit->HasBodySize(InSizeName))
				{
					if (InSizeName != FName(NAME_None).ToString())
					{
						// Only show a warning when the size name is non empty/ not from a NAME_None FName
						Context.Warning(FString::Printf(TEXT("The given body size [%s] doesn't exist in the input Outfit."), *InSizeName), this, Out);
					}
					InSizeName.Reset();
				}

				if (InSizeName.IsEmpty())
				{
					if (const TObjectPtr<const USkeletalMesh>& InTargetBody = GetValue(Context, &TargetBody))
					{
						// Selection by matching target body
						const int32 ClosestBodySize = InOutfitFacade.FindClosestBodySize(*InTargetBody);
						InSizeName = InOutfitFacade.GetBodySizeName(ClosestBodySize);  // Empty string when ClosestBodySize == INDEX_NONE
					}
				}

				if (!InSizeName.IsEmpty())
				{
					// Construct a new Outfit with only the input's pieces of the specified sizes
					TObjectPtr<UChaosOutfit> OutOutfit = NewObject<UChaosOutfit>();
					OutOutfit->Append(*InOutfit, InSizeName);

					SetValue<TObjectPtr<const UChaosOutfit>>(Context, OutOutfit, &Outfit);
					SetValue(Context, OutOutfit->GetOutfitCollection(), &OutfitCollection);
					SetValue(Context, InSizeName, &SizeName);
					return;
				}
			}
			// No selection made, forward the input
			SafeForwardInput(Context, &Outfit, &Outfit);
			SetValue(Context, InOutfit->GetOutfitCollection(), &OutfitCollection);
			SetValue(Context, FString(), &SizeName);
		}
		else
		{
			// No input Outfit, make up an empty output
			TObjectPtr<UChaosOutfit> OutOutfit = NewObject<UChaosOutfit>();
			SetValue<TObjectPtr<const UChaosOutfit>>(Context, OutOutfit, &Outfit);
			SetValue(Context, OutOutfit->GetOutfitCollection(), &OutfitCollection);
			SetValue(Context, FString(), &SizeName);
		}
	}
}

#undef LOCTEXT_NAMESPACE
