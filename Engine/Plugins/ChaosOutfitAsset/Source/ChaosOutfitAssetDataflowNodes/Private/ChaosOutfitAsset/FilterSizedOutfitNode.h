// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "FilterSizedOutfitNode.generated.h"

class UChaosOutfit;
class USkeletalMesh;

/**
 * Select a single size for the passed Outfit and filter out all non matching sizes.
 */
USTRUCT(Meta = (DataflowOutfit))
struct FChaosOutfitAssetFilterSizedOutfitNode final : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosOutfitAssetFilterSizedOutfitNode, "FilterSizedOutfit", "Outfit", "Filter Sized Outfit")

public:
	FChaosOutfitAssetFilterSizedOutfitNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** The Outfit to filter. */
	UPROPERTY(Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Outfit"))
	TObjectPtr<const UChaosOutfit> Outfit;

	/** The Outfit Collection output, provided for convenience as a view into the Outfit object metadata. */
	UPROPERTY(Meta = (DataflowOutput))
	FManagedArrayCollection OutfitCollection;

	/**
	 * The name of the body size to use to filter.
	 * If the input Size Name is empty, the output will be set to a name set from the Target Body's measurements.
	 */
	UPROPERTY(EditAnywhere, Category = "FilterSizedOutfit", Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "SizeName"))
	FString SizeName;

	/**
	 * The target body Skeletal Mesh containing the measurements to select the required size to use to filter.
	 * The target body is unused if Size Name is a valid name.
	 */
	UPROPERTY(EditAnywhere, Category = "FilterSizedOutfit", Meta = (DataflowInput))
	TObjectPtr<const USkeletalMesh> TargetBody;
};
