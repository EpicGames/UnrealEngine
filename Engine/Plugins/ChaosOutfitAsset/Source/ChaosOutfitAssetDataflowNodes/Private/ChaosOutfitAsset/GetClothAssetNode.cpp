// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosOutfitAsset/GetClothAssetNode.h"
#include "ChaosClothAsset/ClothAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GetClothAssetNode)

#define LOCTEXT_NAMESPACE "ChaosGetClothAssetNode"

FChaosGetClothAssetNode::FChaosGetClothAssetNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&ClothAsset);
}

void FChaosGetClothAssetNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TObjectPtr<const UChaosClothAsset>>(&ClothAsset))
	{
		SetValue(Context, ClothAsset, &ClothAsset);
	}
}

#undef LOCTEXT_NAMESPACE
