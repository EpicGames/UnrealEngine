// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GetClothAssetNode.generated.h"

class UChaosClothAsset;

/**
 * Get a Cloth Asset object into the graph.
 */
USTRUCT(Meta = (DataflowOutfit))
struct FChaosGetClothAssetNode final : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosGetClothAssetNode, "GetClothAsset", "Outfit", "Get Cloth Asset")

public:
	FChaosGetClothAssetNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** The Cloth Asset to import into a collection. */
	UPROPERTY(EditAnywhere, Category = "GetClothAsset", Meta = (DataflowOutput))
	TObjectPtr<const UChaosClothAsset> ClothAsset;
};
