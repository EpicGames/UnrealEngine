// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosOutfitAsset/GetOrMakeOutfitFromAssetNode.h"
#include "ChaosOutfitAsset/OutfitAsset.h"
#include "ChaosOutfitAsset/Outfit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GetOrMakeOutfitFromAssetNode)

#define LOCTEXT_NAMESPACE "ChaosOutfitAssetGetOrMakeOutfitFromAssetNode"

FChaosOutfitAssetGetOrMakeOutfitFromAssetNode::FChaosOutfitAssetGetOrMakeOutfitFromAssetNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&OutfitAsset);
	RegisterOutputConnection(&Outfit);
}

void FChaosOutfitAssetGetOrMakeOutfitFromAssetNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Outfit))
	{
		TObjectPtr<UChaosOutfit> OutOutfit = NewObject<UChaosOutfit>();
		if (const UChaosOutfitAsset* const InOutfitAsset = GetValue(Context, &OutfitAsset))
		{
#if WITH_EDITORONLY_DATA
			if (TObjectPtr<const UChaosOutfit> AssetOutfit = InOutfitAsset->GetOutfit())
			{
				SetValue(Context, AssetOutfit, &Outfit);
				return;
			}
#endif

			OutOutfit->Add(*InOutfitAsset);
		}
		SetValue<TObjectPtr<const UChaosOutfit>>(Context, OutOutfit, &Outfit);
	}
}
#undef LOCTEXT_NAMESPACE
