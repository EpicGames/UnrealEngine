// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosOutfitAsset/ClothAssetAnyType.h"
#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GetOrMakeOutfitFromAssetNode.generated.h"

class UChaosOutfitAsset;
class UChaosOutfit;

/**
 * Extract the Outfit from an Outfit Asset.
 * If the Outfit does not exist (e.g., this OutfitAsset has been cooked), recreate a new one.
 */
USTRUCT(Meta = (DataflowOutfit))
struct FChaosOutfitAssetGetOrMakeOutfitFromAssetNode final : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosOutfitAssetGetOrMakeOutfitFromAssetNode, "GetOrMakeOutfitFromAsset", "Outfit", "Get Make Outfit")

public:
	FChaosOutfitAssetGetOrMakeOutfitFromAssetNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** The Outfit Asset input. */
	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<const UChaosOutfitAsset> OutfitAsset;

	/** The Outfit output. */
	UPROPERTY(Meta = (DataflowOutput))
	TObjectPtr<const UChaosOutfit> Outfit;
};
