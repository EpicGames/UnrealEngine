// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosOutfitAsset/GetOutfitAssetNode.h"
#include "ChaosOutfitAsset/OutfitAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GetOutfitAssetNode)

#define LOCTEXT_NAMESPACE "ChaosGetOutfitAssetNode"

FChaosGetOutfitAssetNode::FChaosGetOutfitAssetNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&OutfitAsset);
}

void FChaosGetOutfitAssetNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TObjectPtr<const UChaosOutfitAsset>>(&OutfitAsset))
	{
		SetValue(Context, OutfitAsset, &OutfitAsset);
	}
}

#undef LOCTEXT_NAMESPACE
