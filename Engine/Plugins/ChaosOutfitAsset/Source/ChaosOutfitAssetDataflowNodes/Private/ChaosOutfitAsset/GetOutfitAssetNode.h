// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GetOutfitAssetNode.generated.h"

class UChaosOutfitAsset;

/**
 * Get an Outfit Asset object into the graph.
 */
USTRUCT(Meta = (DataflowOutfit))
struct FChaosGetOutfitAssetNode final : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosGetOutfitAssetNode, "GetOutfitAsset", "Outfit", "Get Outfit Asset")

public:
	FChaosGetOutfitAssetNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** The Outfit Asset to import into a collection. */
	UPROPERTY(EditAnywhere, Category = "GetOutfitAsset", Meta = (DataflowOutput))
	TObjectPtr<const UChaosOutfitAsset> OutfitAsset;
};
