// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosOutfitAsset/GetOutfitBodyPartsNode.h"
#include "ChaosOutfitAsset/Outfit.h"
#include "ChaosOutfitAsset/CollectionOutfitFacade.h"
#include "Engine/SkeletalMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GetOutfitBodyPartsNode)

#define LOCTEXT_NAMESPACE "ChaosGetOutfitBodyPartsNode"

FChaosGetOutfitBodyPartsNode::FChaosGetOutfitBodyPartsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Outfit);
	RegisterOutputConnection(&Outfit, &Outfit);
	RegisterOutputConnection(&BodySizeParts);
}

void FChaosGetOutfitBodyPartsNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Outfit) || Out->IsA(&BodySizeParts))
	{
		TObjectPtr<const UChaosOutfit> OutOutfit;
		if (const TObjectPtr<const UChaosOutfit>& InOutfit = GetValue(Context, &Outfit))
		{
			OutOutfit = InOutfit;
			SafeForwardInput(Context, &Outfit, &Outfit);
		}
		else
		{
			// No input Outfit, make up an empty output
			OutOutfit = NewObject<UChaosOutfit>();
			SetValue(Context, OutOutfit, &Outfit);
		}
		using namespace UE::Chaos::OutfitAsset;

		const FCollectionOutfitConstFacade OutfitFacade(OutOutfit->GetOutfitCollection());
		const int32 NumBodySizes = OutfitFacade.GetNumBodySizes();

		TArray<FChaosOutfitBodySizeBodyParts> OutBodyParts;
		OutBodyParts.SetNum(NumBodySizes);
		for (int32 BodySizeIndex = 0; BodySizeIndex < NumBodySizes; ++BodySizeIndex)
		{
			const TConstArrayView<FSoftObjectPath> BodyPartNames = OutfitFacade.GetBodySizeBodyPartsSkeletalMeshPaths(BodySizeIndex);
			OutBodyParts[BodySizeIndex].BodyParts.Reserve(BodyPartNames.Num());

			for (const FSoftObjectPath& BodyPartName : BodyPartNames)
			{
				if (const TObjectPtr<const USkeletalMesh> SKM = Cast<USkeletalMesh>(BodyPartName.TryLoad()))
				{
					OutBodyParts[BodySizeIndex].BodyParts.Add(SKM);
				}
			}
		}
		SetValue(Context, MoveTemp(OutBodyParts), &BodySizeParts);
	}
}

FChaosExtractBodyPartsArrayFromBodySizePartsNode::FChaosExtractBodyPartsArrayFromBodySizePartsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&BodySizeParts);
	RegisterOutputConnection(&BodyParts);
}

void FChaosExtractBodyPartsArrayFromBodySizePartsNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&BodyParts))
	{
		const FChaosOutfitBodySizeBodyParts& InBodySizeParts = GetValue(Context, &BodySizeParts);
		SetValue(Context, InBodySizeParts.BodyParts, &BodyParts);
	}
}

#undef LOCTEXT_NAMESPACE
