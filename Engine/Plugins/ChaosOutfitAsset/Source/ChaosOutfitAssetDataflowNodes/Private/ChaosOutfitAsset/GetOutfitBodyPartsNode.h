// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GetOutfitBodyPartsNode.generated.h"

class UChaosOutfit;
class USkeletalMesh;

/**
 * Skeletal Mesh body parts for a single body size.
 */
USTRUCT()
struct FChaosOutfitBodySizeBodyParts
{
	GENERATED_USTRUCT_BODY()
	UPROPERTY()
	TArray<TObjectPtr<const USkeletalMesh>> BodyParts;
};

/**
 * Extract the body part Skeletal Meshes from an Outfit, grouped by body size.
 */
USTRUCT(Meta = (DataflowOutfit))
struct FChaosGetOutfitBodyPartsNode final : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosGetOutfitBodyPartsNode, "GetOutfitBodyParts", "Outfit", "Outfit Body Parts Skeletal Mesh")

public:
	FChaosGetOutfitBodyPartsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** The source Outfit. */
	UPROPERTY(Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Outfit"))
	TObjectPtr<const UChaosOutfit> Outfit;

	/** The Outfit body parts grouped by body size. */
	UPROPERTY(Meta = (DataflowOutput))
	TArray<FChaosOutfitBodySizeBodyParts> BodySizeParts;
};

/**
 * Extract the array of Skeletal Mesh body parts for a single body size.
 */
USTRUCT(Meta = (DataflowOutfit))
struct FChaosExtractBodyPartsArrayFromBodySizePartsNode final : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosExtractBodyPartsArrayFromBodySizePartsNode, "ExtractBodyPartsArrayFromBodySizeParts", "Outfit", "Extract Outfit Body Parts Skeletal Mesh")

public:
	FChaosExtractBodyPartsArrayFromBodySizePartsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** The source Outfit body parts for a single body size. */
	UPROPERTY(Meta = (DataflowInput))
	FChaosOutfitBodySizeBodyParts BodySizeParts;

	/** The Outfit body part Skeletal Meshes for the input body size. */
	UPROPERTY(Meta = (DataflowOutput))
	TArray<TObjectPtr<const USkeletalMesh>> BodyParts;
};
