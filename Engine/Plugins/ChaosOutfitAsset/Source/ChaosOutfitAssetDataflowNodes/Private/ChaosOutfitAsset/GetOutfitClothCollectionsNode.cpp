// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosOutfitAsset/GetOutfitClothCollectionsNode.h"
#include "ChaosOutfitAsset/Outfit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GetOutfitClothCollectionsNode)

#define LOCTEXT_NAMESPACE "ChaosGetOutfitClothCollectionsNode"

FChaosGetOutfitClothCollectionsNode::FChaosGetOutfitClothCollectionsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Outfit);
	RegisterInputConnection(&LodIndex);
	RegisterOutputConnection(&Outfit, &Outfit);
	RegisterOutputConnection(&ClothCollections);
	RegisterOutputConnection(&NumLods);
	RegisterOutputConnection(&NumPieces);
}

void FChaosGetOutfitClothCollectionsNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Outfit) || Out->IsA(&ClothCollections) || Out->IsA(&NumLods))
	{
		TObjectPtr<const UChaosOutfit> OutOutfit;
		if (const TObjectPtr<const UChaosOutfit>& InOutfit = GetValue(Context, &Outfit))
		{
			OutOutfit = InOutfit;
			SafeForwardInput(Context, &Outfit, &Outfit);
		}
		else
		{
			// No input Outfit, make up an empty output
			OutOutfit = NewObject<UChaosOutfit>();
			SetValue(Context, OutOutfit, &Outfit);
		}

		const int32 InLODIndex = GetValue(Context, &LodIndex);
		const int32 OutNumLods = (InLODIndex == INDEX_NONE) ? OutOutfit->GetNumLods() : FMath::Min(1, OutOutfit->GetNumLods());
		const int32 OutNumPieces = OutOutfit->GetPieces().Num();

		TArray<FManagedArrayCollection> OutClothCollections;
		OutClothCollections.Reserve(OutNumLods * OutNumPieces);
		for (const TSharedRef<const FManagedArrayCollection>& ClothCollection : OutOutfit->GetClothCollections(InLODIndex))
		{
			// Copy collection
			OutClothCollections.Emplace(*ClothCollection);
		}
		SetValue(Context, MoveTemp(OutClothCollections), &ClothCollections);
		SetValue(Context, OutNumLods, &NumLods);
		SetValue(Context, OutNumPieces, &NumPieces);
	}
}

#undef LOCTEXT_NAMESPACE
