// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GetOutfitClothCollectionsNode.generated.h"

class UChaosOutfit;

/**
 * Extract the Cloth Collections contained into the specified source Outfit.
 */
USTRUCT(Meta = (Experimental, DataflowOutfit))
struct FChaosGetOutfitClothCollectionsNode final : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosGetOutfitClothCollectionsNode, "GetOutfitClothCollections", "Outfit", "Outfit Cloth Collections")

public:
	FChaosGetOutfitClothCollectionsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** The source Outfit. */
	UPROPERTY(Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Outfit"))
	TObjectPtr<const UChaosOutfit> Outfit;

	/** The outfit cloth collections array in group of consecutives LODs for each of the outfit pieces. */
	UPROPERTY(Meta = (DataflowOutput))
	TArray<FManagedArrayCollection> ClothCollections;

	/** The LOD to output in the cloth collections array. Set to -1 to output all LODs */
	UPROPERTY(EditAnywhere, Category = "Get Outfit Cloth Collections", DisplayName = "LOD Index", Meta = (UIMin = -1, DataflowInput))
	int32 LodIndex = -1;

	/** The number of LODs output in the cloth collections array. */
	UPROPERTY(DisplayName = "Num LODs", Meta = (DataflowOutput))
	int32 NumLods = 0;

	/** The number of cloth pieces output in the cloth collections array. */
	UPROPERTY(Meta = (DataflowOutput))
	int32 NumPieces = 0;
};
