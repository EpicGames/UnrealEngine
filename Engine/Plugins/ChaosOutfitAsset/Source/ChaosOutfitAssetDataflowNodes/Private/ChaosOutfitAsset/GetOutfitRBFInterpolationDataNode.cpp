// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosOutfitAsset/GetOutfitRBFInterpolationDataNode.h"
#include "ChaosOutfitAsset/Outfit.h"
#include "ChaosOutfitAsset/CollectionOutfitFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GetOutfitRBFInterpolationDataNode)

#define LOCTEXT_NAMESPACE "ChaosGetOutfitRBFInterpolationDataNode"

FChaosGetOutfitRBFInterpolationDataNode::FChaosGetOutfitRBFInterpolationDataNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Outfit);
	RegisterInputConnection(&BodySizeIndex);
	RegisterInputConnection(&BodyPartIndex);
	RegisterOutputConnection(&Outfit, &Outfit);
	RegisterOutputConnection(&InterpolationData);
}

void FChaosGetOutfitRBFInterpolationDataNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Outfit) || Out->IsA(&InterpolationData))
	{
		FMeshResizingRBFInterpolationData OutInterpolationData;
		if (const TObjectPtr<const UChaosOutfit>& InOutfit = GetValue(Context, &Outfit))
		{
			using namespace UE::Chaos::OutfitAsset;
			const FCollectionOutfitConstFacade OutfitFacade(InOutfit->GetOutfitCollection());

			const int32 InBodySizeIndex = GetValue(Context, &BodySizeIndex);
			const int32 InBodyPartIndex = GetValue(Context, &BodyPartIndex);
			if (InBodySizeIndex >= 0 && InBodySizeIndex < OutfitFacade.GetNumBodySizes())
			{
				const FCollectionOutfitConstFacade::FRBFInterpolationDataWrapper InterpolationDataWrapper = OutfitFacade.GetBodySizeInterpolationData(InBodySizeIndex);
				if (InterpolationDataWrapper.SampleIndices.IsValidIndex(InBodyPartIndex))
				{
					OutInterpolationData.SampleIndices = InterpolationDataWrapper.SampleIndices[InBodyPartIndex];
					OutInterpolationData.SampleRestPositions = InterpolationDataWrapper.SampleRestPositions[InBodyPartIndex];
					OutInterpolationData.InterpolationWeights = InterpolationDataWrapper.InterpolationWeights[InBodyPartIndex];
				}
				else
				{
					Context.Warning(FString::Printf(TEXT("The given BodyPartIndex [%d] doesn't index into this BodySize's NumBodyParts [%d]"), InBodyPartIndex, InterpolationDataWrapper.SampleIndices.Num()), this, Out);
				}
			}
			else
			{
				Context.Warning(FString::Printf(TEXT("The given BodySizeIndex [%d] doesn't index into the NumBodySizes [%d]"), InBodySizeIndex, OutfitFacade.GetNumBodySizes()), this, Out);
			}

		}
		SafeForwardInput(Context, &Outfit, &Outfit);
		SetValue(Context, MoveTemp(OutInterpolationData), &InterpolationData);
	}
}


#undef LOCTEXT_NAMESPACE
