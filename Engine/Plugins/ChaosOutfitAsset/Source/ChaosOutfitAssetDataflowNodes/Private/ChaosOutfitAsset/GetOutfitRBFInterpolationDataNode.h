// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "MeshResizing/RBFInterpolation.h"
#include "GetOutfitRBFInterpolationDataNode.generated.h"

class UChaosOutfit;

/**
 * Extract the Body Part RBF Interpolation Data from an Outfit.
 */
USTRUCT(Meta = (DataflowOutfit))
struct FChaosGetOutfitRBFInterpolationDataNode final : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosGetOutfitRBFInterpolationDataNode, "GetOutfitRBFInterpolationData", "Outfit", "Outfit RBF Interpolation Data")

public:
	FChaosGetOutfitRBFInterpolationDataNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** The source Outfit. */
	UPROPERTY(Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Outfit"))
	TObjectPtr<const UChaosOutfit> Outfit;

	/** The body size index.*/
	UPROPERTY(EditAnywhere, Category = "Outfit RBF Interpolation", Meta = (DataflowInput, ClampMin = 0))
	int32 BodySizeIndex = 0;

	/** The body part (sub) index.*/
	UPROPERTY(EditAnywhere, Category = "Outfit RBF Interpolation", Meta = (DataflowInput, ClampMin = 0))
	int32 BodyPartIndex = 0;

	/** The interpolation data. */
	UPROPERTY(Meta = (DataflowOutput))
	FMeshResizingRBFInterpolationData InterpolationData;
};