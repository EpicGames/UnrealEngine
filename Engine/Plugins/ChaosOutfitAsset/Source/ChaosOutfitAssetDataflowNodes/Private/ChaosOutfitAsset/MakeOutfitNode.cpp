// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosOutfitAsset/MakeOutfitNode.h"
#include "ChaosOutfitAsset/ClothAssetAnyType.h"
#include "ChaosOutfitAsset/Outfit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MakeOutfitNode)

#define LOCTEXT_NAMESPACE "ChaosOutfitAssetMakeOutfitNode"

FChaosOutfitAssetMakeOutfitNode::FChaosOutfitAssetMakeOutfitNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&Outfit);

	for (int32 Index = 0; Index < NumInitialClothAssets; ++Index)
	{
		AddPins();
	}
	check(GetNumInputs() == NumRequiredInputs + NumInitialClothAssets);  // Update NumRequiredInputs when adding inputs (used by Serialize)
}

void FChaosOutfitAssetMakeOutfitNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Outfit))
	{
		TObjectPtr<UChaosOutfit> OutOutfit = NewObject<UChaosOutfit>();

		auto AddAsset = [&OutOutfit](const TObjectPtr<UChaosClothAssetBase>& InAsset)
			{
				if (InAsset && InAsset->GetNumClothSimulationModels())  // TODO Use future number of piece API instead
				{
					OutOutfit->Add(*InAsset);
				}
			};

		for (int32 InputIndex = 0; InputIndex < ClothAssets.Num(); ++InputIndex)
		{
			const FChaosClothAssetOrArrayType& InAssetOrArray = GetValue(Context, &ClothAssets[InputIndex]);
			if (!InAssetOrArray.IsArray())
			{
				AddAsset(InAssetOrArray.Get());
			}
			else
			{
				const TArray<TObjectPtr<UChaosClothAssetBase>>& InArray = InAssetOrArray.GetArray();

				for (int32 ElementIndex = 0; ElementIndex < InArray.Num(); ++ElementIndex)
				{
					AddAsset(InArray[ElementIndex]);
				}
			}
		}

		SetValue<TObjectPtr<const UChaosOutfit>>(Context, OutOutfit, &Outfit);
	}
}

TArray<UE::Dataflow::FPin> FChaosOutfitAssetMakeOutfitNode::AddPins()
{
	const int32 Index = ClothAssets.AddDefaulted();
	const FDataflowInput& Input = RegisterInputArrayConnection(GetConnectionReference(Index));
	return { { UE::Dataflow::FPin::EDirection::INPUT, Input.GetType(), Input.GetName() } };
}

TArray<UE::Dataflow::FPin> FChaosOutfitAssetMakeOutfitNode::GetPinsToRemove() const
{
	const int32 Index = ClothAssets.Num() - 1;
	check(ClothAssets.IsValidIndex(Index));
	if (const FDataflowInput* const Input = FindInput(GetConnectionReference(Index)))
	{
		return { { UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() } };
	}
	return Super::GetPinsToRemove();
}

void FChaosOutfitAssetMakeOutfitNode::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	const int32 Index = ClothAssets.Num() - 1;
	check(ClothAssets.IsValidIndex(Index));
#if DO_CHECK
	const FDataflowInput* const Input = FindInput(GetConnectionReference(Index));
	check(Input);
	check(Input->GetName() == Pin.Name);
	check(Input->GetType() == Pin.Type);
#endif
	ClothAssets.SetNum(Index);

	return Super::OnPinRemoved(Pin);
}

void FChaosOutfitAssetMakeOutfitNode::PostSerialize(const FArchive& Ar)
{
	// Added pins need to be restored when loading to make sure they get reconnected
	if (Ar.IsLoading())
	{
		for (int32 Index = 0; Index < NumInitialClothAssets; ++Index)
		{
			check(FindInput(GetConnectionReference(Index)));
		}

		for (int32 Index = NumInitialClothAssets; Index < ClothAssets.Num(); ++Index)
		{
			FindOrRegisterInputArrayConnection(GetConnectionReference(Index));
		}
		if (Ar.IsTransacting())
		{
			const int32 OrigNumRegisteredInputs = GetNumInputs();
			check(OrigNumRegisteredInputs >= NumRequiredInputs + NumInitialClothAssets);
			const int32 OrigNumClothAssets = ClothAssets.Num();
			const int32 OrigNumRegisteredClothAssets = (OrigNumRegisteredInputs - NumRequiredInputs);
			if (OrigNumRegisteredClothAssets > OrigNumClothAssets)
			{
				// Inputs have been removed, temporarily expand ClothAssets so we can get connection references
				ClothAssets.SetNum(OrigNumRegisteredClothAssets);
				for (int32 Index = OrigNumClothAssets; Index < ClothAssets.Num(); ++Index)
				{
					UnregisterInputConnection(GetConnectionReference(Index));
				}
				ClothAssets.SetNum(OrigNumClothAssets);
			}
		}
		else
		{
			ensureAlways(ClothAssets.Num() + NumRequiredInputs == GetNumInputs());
		}
	}
}

UE::Dataflow::TConnectionReference<FChaosClothAssetOrArrayAnyType> FChaosOutfitAssetMakeOutfitNode::GetConnectionReference(int32 Index) const
{
	return { &ClothAssets[Index], Index, &ClothAssets };
}

#undef LOCTEXT_NAMESPACE
