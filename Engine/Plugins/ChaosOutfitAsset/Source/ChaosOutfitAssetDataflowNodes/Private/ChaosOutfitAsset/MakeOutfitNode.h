// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosOutfitAsset/ClothAssetAnyType.h"
#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "MakeOutfitNode.generated.h"

class UChaosClothAssetBase;
class UChaosOutfit;

/**
 * Make multiple Cloth Asset objects into an Outfit.
 */
USTRUCT(Meta = (DataflowOutfit))
struct FChaosOutfitAssetMakeOutfitNode final : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosOutfitAssetMakeOutfitNode, "MakeOutfit", "Outfit", "Cloth Asset Make Outfit")

public:
	FChaosOutfitAssetMakeOutfitNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual TArray<UE::Dataflow::FPin> AddPins() override;
	virtual bool CanAddPin() const override
	{
		return true;
	}
	virtual bool CanRemovePin() const override
	{
		return ClothAssets.Num() > NumInitialClothAssets;
	}
	virtual TArray<UE::Dataflow::FPin> GetPinsToRemove() const override;
	virtual void OnPinRemoved(const UE::Dataflow::FPin& Pin) override;
	virtual void PostSerialize(const FArchive& Ar) override;
	//~ End FDataflowNode interface

	UE::Dataflow::TConnectionReference<FChaosClothAssetOrArrayAnyType> GetConnectionReference(int32 Index) const;

	static constexpr int32 NumRequiredInputs = 0;
	static constexpr int32 NumInitialClothAssets = 1;

	/** The Outfit output. */
	UPROPERTY(Meta = (DataflowOutput))
	TObjectPtr<const UChaosOutfit> Outfit;

	/** The Cloth or Outfit Assets to import into a new Outfit. */
	UPROPERTY()
	TArray<FChaosClothAssetOrArrayAnyType> ClothAssets;
};
