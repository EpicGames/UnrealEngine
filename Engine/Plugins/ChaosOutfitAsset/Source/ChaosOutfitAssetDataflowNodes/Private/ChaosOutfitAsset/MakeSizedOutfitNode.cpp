// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosOutfitAsset/MakeSizedOutfitNode.h"
#include "ChaosOutfitAsset/CollectionOutfitFacade.h"
#include "ChaosOutfitAsset/ClothAssetAnyType.h"
#include "ChaosOutfitAsset/Outfit.h"
#include "Engine/SkeletalMesh.h"
#include "Misc/ScopedSlowTask.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MakeSizedOutfitNode)

#define LOCTEXT_NAMESPACE "ChaosOutfitAssetMakeSizedOutfitNode"

FChaosOutfitAssetMakeSizedOutfitNode::FChaosOutfitAssetMakeSizedOutfitNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&Outfit);
	RegisterOutputConnection(&OutfitCollection);

	for (int32 Index = 0; Index < NumInitialSizedOutfitSources; ++Index)
	{
		AddPins();
	}
	check(GetNumInputs() == NumRequiredInputs + NumInitialSizedOutfitSources);  // Update NumRequiredInputs when adding inputs (used by Serialize)
}

void FChaosOutfitAssetMakeSizedOutfitNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Chaos::OutfitAsset;

	if (Out->IsA(&Outfit) || Out->IsA(&OutfitCollection))
	{
		TObjectPtr<UChaosOutfit> OutOutfit = NewObject<UChaosOutfit>();
		UE::Chaos::OutfitAsset::FCollectionOutfitFacade OutfitFacade(OutOutfit->GetOutfitCollection());

		// Make a new GUID for this sized outfit asset
		const FGuid OutfitGuid = FGuid::NewGuid();

		// Calculate the number of source to add
		int32 NumOutfitSources = 0;
		for (int32 SourceIndex = 0; SourceIndex < SizedOutfitSources.Num(); ++SourceIndex)
		{
			const FChaosSizedOutfitSourceOrArrayType& SizedOutfitSourceOrArray = GetValue(Context, &SizedOutfitSources[SourceIndex]);
			for (const FChaosSizedOutfitSource& SizedOutfitSource : SizedOutfitSourceOrArray.Array)
			{
				if (SizedOutfitSource.SourceAsset || !SizedOutfitSource.GetBodySizeName().IsEmpty())
				{
					++NumOutfitSources;
				}
			}
		}

		FScopedSlowTask SlowTask((float)NumOutfitSources, LOCTEXT("AddOutfitSources", "Adding sized outfit sources..."));
		SlowTask.MakeDialogDelayed(0.01f);

		// Add outfit sources
		for (int32 SourceIndex = 0; SourceIndex < SizedOutfitSources.Num(); ++SourceIndex)
		{
			const FChaosSizedOutfitSourceOrArrayType& SizedOutfitSourceOrArray = GetValue(Context, &SizedOutfitSources[SourceIndex]);

			for (const FChaosSizedOutfitSource& SizedOutfitSource : SizedOutfitSourceOrArray.Array)
			{
				FString BodySizeName = SizedOutfitSource.GetBodySizeName();
				FString SourceAssetName;

				if (!SizedOutfitSource.SourceAsset)
				{
					if (BodySizeName.IsEmpty())
					{
						continue;
					}
					Context.Warning(FString::Printf(TEXT("Empty source asset specified for size [%s]."), *BodySizeName), this, Out);
					SourceAssetName = TEXT("-");
				}
				else
				{
					SourceAssetName = SizedOutfitSource.SourceAsset->GetName();
					if (BodySizeName.IsEmpty())
					{
						Context.Info(FString::Printf(TEXT("The source asset [%s] has no body size, and therefore will use the default body size."), *SourceAssetName), this, Out);
						BodySizeName = DefaultBodySize.ToString();
					}
				}
				if (OutfitFacade.HasBodySize(BodySizeName))
				{
					Context.Warning(FString::Printf(TEXT("The body size [%s] already existed and had to be overwritten."), *BodySizeName), this, Out);
				}

				SlowTask.EnterProgressFrame(1.f, 
					FText::Format(LOCTEXT("AddOutfitSource", "Adding source asset [{0}] for body size [{1}]..."), 
						FText::FromString(SourceAssetName),
						FText::FromString(BodySizeName)));
				SlowTask.TickProgress();  // The RBF weight evaluations are really slow
				SlowTask.ForceRefresh();  // ForceRefresh could be overkill, but the progress still doesn't show in some instances

				OutOutfit->Add(SizedOutfitSource, OutfitGuid);
			}
		}

		SetValue<TObjectPtr<const UChaosOutfit>>(Context, OutOutfit, &Outfit);
		SetValue(Context, OutOutfit->GetOutfitCollection(), &OutfitCollection);
	}
}

TArray<UE::Dataflow::FPin> FChaosOutfitAssetMakeSizedOutfitNode::AddPins()
{
	const int32 Index = SizedOutfitSources.AddDefaulted();
	const FDataflowInput& Input = RegisterInputArrayConnection(GetConnectionReference(Index));
	return { { UE::Dataflow::FPin::EDirection::INPUT, Input.GetType(), Input.GetName() } };
}

TArray<UE::Dataflow::FPin> FChaosOutfitAssetMakeSizedOutfitNode::GetPinsToRemove() const
{
	const int32 Index = SizedOutfitSources.Num() - 1;
	check(SizedOutfitSources.IsValidIndex(Index));
	if (const FDataflowInput* const Input = FindInput(GetConnectionReference(Index)))
	{
		return { { UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() } };
	}
	return Super::GetPinsToRemove();
}

void FChaosOutfitAssetMakeSizedOutfitNode::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	const int32 Index = SizedOutfitSources.Num() - 1;
	check(SizedOutfitSources.IsValidIndex(Index));
#if DO_CHECK
	const FDataflowInput* const Input = FindInput(GetConnectionReference(Index));
	check(Input);
	check(Input->GetName() == Pin.Name);
	check(Input->GetType() == Pin.Type);
#endif
	SizedOutfitSources.SetNum(Index);

	return Super::OnPinRemoved(Pin);
}

void FChaosOutfitAssetMakeSizedOutfitNode::PostSerialize(const FArchive& Ar)
{
	// Added pins need to be restored when loading to make sure they get reconnected
	if (Ar.IsLoading())
	{
		for (int32 Index = 0; Index < NumInitialSizedOutfitSources; ++Index)
		{
			check(FindInput(GetConnectionReference(Index)));
		}

		for (int32 Index = NumInitialSizedOutfitSources; Index < SizedOutfitSources.Num(); ++Index)
		{
			FindOrRegisterInputArrayConnection(GetConnectionReference(Index));
		}
		if (Ar.IsTransacting())
		{
			const int32 OrigNumRegisteredInputs = GetNumInputs();
			check(OrigNumRegisteredInputs >= NumRequiredInputs + NumInitialSizedOutfitSources);
			const int32 OrigNumSizedOutfitSources = SizedOutfitSources.Num();
			const int32 OrigNumRegisteredSizedOutfitSources = (OrigNumRegisteredInputs - NumRequiredInputs);
			if (OrigNumRegisteredSizedOutfitSources > OrigNumSizedOutfitSources)
			{
				// Inputs have been removed, temporarily expand SizedOutfitInputs so we can get connection references
				SizedOutfitSources.SetNum(OrigNumRegisteredSizedOutfitSources);
				for (int32 Index = OrigNumSizedOutfitSources; Index < SizedOutfitSources.Num(); ++Index)
				{
					UnregisterInputConnection(GetConnectionReference(Index));
				}
				SizedOutfitSources.SetNum(OrigNumSizedOutfitSources);
			}
		}
		else
		{
			ensureAlways(SizedOutfitSources.Num() + NumRequiredInputs == GetNumInputs());
		}
	}
}

UE::Dataflow::TConnectionReference<FChaosSizedOutfitSourceOrArrayAnyType> FChaosOutfitAssetMakeSizedOutfitNode::GetConnectionReference(int32 Index) const
{
	return { &SizedOutfitSources[Index], Index, &SizedOutfitSources };
}

#undef LOCTEXT_NAMESPACE
