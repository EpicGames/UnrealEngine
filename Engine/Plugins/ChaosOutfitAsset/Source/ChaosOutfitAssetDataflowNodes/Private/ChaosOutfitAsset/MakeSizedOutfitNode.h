// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosOutfitAsset/SizedOutfitSourceAnyType.h"
#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "MakeSizedOutfitNode.generated.h"

class UChaosOutfit;

/**
 * Make multiple Cloth Assets into a sized Outfit.
 */
USTRUCT(Meta = ( DataflowOutfit))
struct FChaosOutfitAssetMakeSizedOutfitNode final : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosOutfitAssetMakeSizedOutfitNode, "MakeSizedOutfit", "Outfit", "Cloth Asset Make Sized Outfit")

public:
	FChaosOutfitAssetMakeSizedOutfitNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual TArray<UE::Dataflow::FPin> AddPins() override;
	virtual bool CanAddPin() const override
	{
		return true;
	}
	virtual bool CanRemovePin() const override
	{
		return SizedOutfitSources.Num() > NumInitialSizedOutfitSources;
	}
	virtual TArray<UE::Dataflow::FPin> GetPinsToRemove() const override;
	virtual void OnPinRemoved(const UE::Dataflow::FPin& Pin) override;
	virtual void PostSerialize(const FArchive& Ar) override;
	//~ End FDataflowNode interface

	UE::Dataflow::TConnectionReference<FChaosSizedOutfitSourceOrArrayAnyType> GetConnectionReference(int32 Index) const;

	static constexpr int32 NumRequiredInputs = 0;
	static constexpr int32 NumInitialSizedOutfitSources = 1;

	/** The Outfit output. */
	UPROPERTY(Meta = (DataflowOutput))
	TObjectPtr<const UChaosOutfit> Outfit;

	/** The Outfit Collection output, provided for convenience as a view into the Outfit object metadata. */
	UPROPERTY(Meta = (DataflowOutput))
	FManagedArrayCollection OutfitCollection;

	/** The Cloth/Outfit assets and body references to import into as a new sized Outfit. */
	UPROPERTY()
	TArray<FChaosSizedOutfitSourceOrArrayAnyType> SizedOutfitSources;
};
