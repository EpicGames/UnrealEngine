// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosOutfitAsset/MergeOutfitsNode.h"
#include "ChaosOutfitAsset/Outfit.h"
#include "Dataflow/DataflowInputOutput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MergeOutfitsNode)

#define LOCTEXT_NAMESPACE "ChaosOutfitAssetMergeOutfitsNode"

FChaosOutfitAssetMergeOutfitsNode::FChaosOutfitAssetMergeOutfitsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	check(GetNumInputs() == NumRequiredInputs);
	for (int32 Index = 0; Index < NumInitialOptionalInputs; ++Index)
	{
		AddPins();
	}
	RegisterOutputConnection(&Outfit)
		.SetPassthroughInput(GetConnectionReference(0));
}

void FChaosOutfitAssetMergeOutfitsNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TObjectPtr<const UChaosOutfit>>(&Outfit))
	{
		TObjectPtr<UChaosOutfit> OutOutfit;

		for (int32 Index = 0; Index < Outfits.Num(); ++Index)
		{
			const TObjectPtr<const UChaosOutfit>& InOutfit = GetValue(Context, &Outfits[Index]);
			if (InOutfit && InOutfit->GetPieces().Num())
			{
				if (!OutOutfit)
				{
					OutOutfit = NewObject<UChaosOutfit>();
				}
				OutOutfit->Append(*InOutfit);
			}
		}

		if (OutOutfit)
		{
			SetValue<TObjectPtr<const UChaosOutfit>>(Context, OutOutfit, &Outfit);
		}
		else
		{
			SafeForwardInput(Context, GetConnectionReference(0), &Outfit);
		}
	}
}

TArray<UE::Dataflow::FPin> FChaosOutfitAssetMergeOutfitsNode::AddPins()
{
	const int32 Index = Outfits.AddDefaulted();
	const FDataflowInput& Input = RegisterInputArrayConnection(GetConnectionReference(Index));
	return { { UE::Dataflow::FPin::EDirection::INPUT, Input.GetType(), Input.GetName() } };
}

TArray<UE::Dataflow::FPin> FChaosOutfitAssetMergeOutfitsNode::GetPinsToRemove() const
{
	const int32 Index = Outfits.Num() - 1;
	check(Outfits.IsValidIndex(Index));
	if (const FDataflowInput* const Input = FindInput(GetConnectionReference(Index)))
	{
		return { { UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() } };
	}
	return Super::GetPinsToRemove();
}

void FChaosOutfitAssetMergeOutfitsNode::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	const int32 Index = Outfits.Num() - 1;
	check(Outfits.IsValidIndex(Index));
#if DO_CHECK
	const FDataflowInput* const Input = FindInput(GetConnectionReference(Index));
	check(Input);
	check(Input->GetName() == Pin.Name);
	check(Input->GetType() == Pin.Type);
#endif
	Outfits.SetNum(Index);

	return Super::OnPinRemoved(Pin);
}

void FChaosOutfitAssetMergeOutfitsNode::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		for (int32 Index = 0; Index < NumInitialOptionalInputs; ++Index)
		{
			check(FindInput(GetConnectionReference(Index)));
		}
		for (int32 Index = NumInitialOptionalInputs; Index < Outfits.Num(); ++Index)
		{
			FindOrRegisterInputArrayConnection(GetConnectionReference(Index));
		}
		if (Ar.IsTransacting())
		{
			const int32 OrigNumRegisteredInputs = GetNumInputs();
			check(OrigNumRegisteredInputs >= NumRequiredInputs + NumInitialOptionalInputs);
			const int32 OrigNumCollections = Outfits.Num();
			const int32 OrigNumRegisteredOutfits = OrigNumRegisteredInputs - NumRequiredInputs;
			if (OrigNumRegisteredOutfits > OrigNumCollections)
			{
				Outfits.SetNum(GetNumInputs() - 1);
				for (int32 Index = OrigNumCollections; Index < Outfits.Num(); ++Index)
				{
					UnregisterInputConnection(GetConnectionReference(Index));
				}
				Outfits.SetNum(OrigNumCollections);
			}
		}
		else
		{
			ensureAlways(Outfits.Num() == GetNumInputs());
		}
	}
}

UE::Dataflow::TConnectionReference<TObjectPtr<const UChaosOutfit>> FChaosOutfitAssetMergeOutfitsNode::GetConnectionReference(int32 Index) const
{
	return { &Outfits[Index], Index, &Outfits };
}

#undef LOCTEXT_NAMESPACE
