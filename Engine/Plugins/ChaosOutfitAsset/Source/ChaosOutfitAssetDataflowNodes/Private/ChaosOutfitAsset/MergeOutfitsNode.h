// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "MergeOutfitsNode.generated.h"

class UChaosOutfit;

/**
 * Merge multiple Outfits into a single Outfit.
 */
USTRUCT(Meta = (DataflowOutfit))
struct FChaosOutfitAssetMergeOutfitsNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosOutfitAssetMergeOutfitsNode, "MergeOutfits", "Outfit", "Outfit Merge Outfits")

public:
	FChaosOutfitAssetMergeOutfitsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual TArray<UE::Dataflow::FPin> AddPins() override;
	virtual bool CanAddPin() const override
	{
		return true;
	}
	virtual bool CanRemovePin() const override
	{
		return Outfits.Num() > NumInitialOptionalInputs;
	}
	virtual TArray<UE::Dataflow::FPin> GetPinsToRemove() const override;
	virtual void OnPinRemoved(const UE::Dataflow::FPin& Pin) override;
	virtual void PostSerialize(const FArchive& Ar) override;
	//~ End FDataflowNode interface

	UE::Dataflow::TConnectionReference<TObjectPtr<const UChaosOutfit>> GetConnectionReference(int32 Index) const;

	/** The Outfit inputs to merge into a single Outfit. */
	UPROPERTY()
	TArray<TObjectPtr<const UChaosOutfit>> Outfits;

	/** The Outfit output. */
	UPROPERTY(Meta = (DataflowOutput, DataflowPassthrough = "Outfits[0]"))
	TObjectPtr<const UChaosOutfit> Outfit;

	static constexpr int32 NumRequiredInputs = 0;
	static constexpr int32 NumInitialOptionalInputs = 2;
};
