// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosOutfitAsset/OutfitAssetTerminalNode.h"
#if WITH_EDITOR
#include "AssetRegistry/AssetRegistryModule.h"
#endif
#include "ChaosOutfitAsset/OutfitAsset.h"
#include "Dataflow/DataflowInputOutput.h"
#if WITH_EDITOR
#include "Dataflow/DataflowObjectInterface.h"
#include "Engine/SkeletalMesh.h"
#include "Misc/FileHelper.h"
#include "Misc/ScopedSlowTask.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(OutfitAssetTerminalNode)

#define LOCTEXT_NAMESPACE "ChaosOutfitAssetTerminalNode"

namespace UE::Chaos::OutfitAsset::Private
{
#if WITH_EDITOR
	static void ExportToSkeletalMesh(const TObjectPtr<const UChaosOutfitAsset>& OutfitAsset)
	{
		using namespace UE::Geometry;
		if (OutfitAsset)
		{
			// Make a name from the outfit asset name
			const FString SkeletalMeshPath = FPackageName::GetLongPackagePath(OutfitAsset->GetOutermost()->GetName());
			const FString OutfitAssetName = OutfitAsset->GetName();
			FString SkeletalMeshName = FString(TEXT("SK_")) + (OutfitAssetName.StartsWith(TEXT("OA_")) ? OutfitAssetName.RightChop(3) : OutfitAssetName);
			FString SkeletalMeshPackageName = FPaths::Combine(SkeletalMeshPath, SkeletalMeshName);

			// Open a Save As dialog
			FSaveAssetDialogConfig SaveAssetDialogConfig;
			SaveAssetDialogConfig.DefaultPath = SkeletalMeshPath;
			SaveAssetDialogConfig.DefaultAssetName = SkeletalMeshName;
			SaveAssetDialogConfig.AssetClassNames.Add(USkeletalMesh::StaticClass()->GetClassPathName());
			SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;
			SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveAssetDialogConfig", "Convert Outfit Asset To Skeletal Mesh As");

			const FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
			const FString AssetSavePath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);
			const FString AssetSavePackageName = FPackageName::ObjectPathToPackageName(AssetSavePath);
			FText OutError;
			if (AssetSavePath.IsEmpty() || !FFileHelper::IsFilenameValidForSaving(AssetSavePackageName, OutError))
			{
				return;
			}
			SkeletalMeshPackageName = AssetSavePackageName;
			SkeletalMeshName = FPackageName::GetLongPackageAssetName(SkeletalMeshPackageName);

			constexpr float NumSteps = 0.f;
			FScopedSlowTask SlowTask(NumSteps, LOCTEXT("ConvertToSkeletalMesh", "Converting Cloth Outfit to a Skeletal Mesh asset..."));
			SlowTask.MakeDialog();

			UPackage* const SkeletalMeshPackage = CreatePackage(*SkeletalMeshPackageName);
			USkeletalMesh* const SkeletalMesh =
				NewObject<USkeletalMesh>(
					SkeletalMeshPackage,
					USkeletalMesh::StaticClass(),
					FName(SkeletalMeshName),
					RF_Public | RF_Standalone | RF_Transactional);

			// Export to skeletal mesh
			if (SkeletalMesh)
			{
				OutfitAsset->ExportToSkeletalMesh(*SkeletalMesh);
			}

			SkeletalMesh->MarkPackageDirty();

			// Notify the asset registry
			FAssetRegistryModule::AssetCreated(SkeletalMesh);
		}
	}
#endif  // #if WITH_EDITOR
}

FChaosOutfitAssetTerminalNode::FChaosOutfitAssetTerminalNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowTerminalNode(InParam, InGuid)
#if WITH_EDITOR
	, ConvertToSkeletalMesh(
		FDataflowFunctionProperty::FDelegate::CreateLambda([this](UE::Dataflow::FContext& Context)
			{
				if (const UE::Dataflow::FEngineContext* EngineContext = Context.AsType<UE::Dataflow::FEngineContext>())
				{
					if (const UChaosOutfitAsset* const OwnerOutfitAsset = Cast<UChaosOutfitAsset>(EngineContext->Owner))
					{
						UE::Chaos::OutfitAsset::Private::ExportToSkeletalMesh(OwnerOutfitAsset);
					}
				}
			}))
#endif
{
	RegisterInputConnection(&Outfit);
}

void FChaosOutfitAssetTerminalNode::SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const
{
	if (TObjectPtr<UChaosOutfitAsset> OutfitAsset = Cast<UChaosOutfitAsset>(Asset))
	{
		const TObjectPtr<const UChaosOutfit>& InOutfit = GetValue(Context, &Outfit);

		// Build the asset
		OutfitAsset->Build(InOutfit, &Context);

		// Asset must be resaved
		OutfitAsset->MarkPackageDirty();
	}
}

#undef LOCTEXT_NAMESPACE
