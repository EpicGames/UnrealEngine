// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowFunctionProperty.h"
#include "Dataflow/DataflowTerminalNode.h"
#include "OutfitAssetTerminalNode.generated.h"

class UChaosOutfit;

/**
 * Outfit Terminal Node to generate an Outfit Asset.
 */
USTRUCT(Meta = (DataflowOutfit, DataflowTerminal))
struct FChaosOutfitAssetTerminalNode final : public FDataflowTerminalNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosOutfitAssetTerminalNode, "OutfitAssetTerminal", "Outfit", "Outfit Terminal")

public:
	FChaosOutfitAssetTerminalNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual void SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const override;
	virtual void Evaluate(UE::Dataflow::FContext& Context) const override {}
	//~ End FDataflowNode interface

	/** Input Outfit. */
	UPROPERTY(Meta = (DataflowInput))
	TObjectPtr<const UChaosOutfit> Outfit;

	/** Convert this Outfit to a Skeletal Mesh asset. */
	UPROPERTY(EditAnywhere, Category = "Outfit Asset Terminal", Meta = (ButtonImage = "Icons.Convert"))
	FDataflowFunctionProperty ConvertToSkeletalMesh;
};

