// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosOutfitAsset/OutfitQueryNode.h"
#include "ChaosOutfitAsset/CollectionOutfitFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OutfitQueryNode)

#define LOCTEXT_NAMESPACE "ChaosOutfitAssetOutfitQueryNode"

FChaosOutfitAssetOutfitQueryNode::FChaosOutfitAssetOutfitQueryNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Outfit);
	RegisterOutputConnection(&Outfit, &Outfit);
	RegisterOutputConnection(&bHasAnyValidPieces);
	RegisterOutputConnection(&bHasAnyValidBodySizes);
}

void FChaosOutfitAssetOutfitQueryNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Chaos::OutfitAsset;
	if (Out->IsA(&Outfit))
	{
		SafeForwardInput(Context, &Outfit, &Outfit);
	}
	else if (Out->IsA<bool>(&bHasAnyValidPieces))
	{
		if (const TObjectPtr<const UChaosOutfit> InOutfit = GetValue(Context, &Outfit))
		{
			FCollectionOutfitConstFacade OutfitFacade(InOutfit->GetOutfitCollection());
			if (OutfitFacade.IsValid())
			{
				const bool bOutHasAnyValidPieces = !OutfitFacade.GetOutfitPiecesGuids().IsEmpty();
				SetValue(Context, bOutHasAnyValidPieces, &bHasAnyValidPieces);
				return;
			}
		}
		SetValue(Context, false, &bHasAnyValidPieces);
	}
	else if (Out->IsA<bool>(&bHasAnyValidBodySizes))
	{
		if (const TObjectPtr<const UChaosOutfit> InOutfit = GetValue(Context, &Outfit))
		{
			FCollectionOutfitConstFacade OutfitFacade(InOutfit->GetOutfitCollection());
			if (OutfitFacade.IsValid())
			{
				const bool bOutHasAnyValidBodySizes = OutfitFacade.HasValidBodySize(bBodyPartMustExist, bMeasurementsMustExist, bInterpolationDataMustExist);
				SetValue(Context, bOutHasAnyValidBodySizes, &bHasAnyValidBodySizes);
				return;
			}
		}
		SetValue(Context, false, &bHasAnyValidBodySizes);
	}
}

#undef LOCTEXT_NAMESPACE
