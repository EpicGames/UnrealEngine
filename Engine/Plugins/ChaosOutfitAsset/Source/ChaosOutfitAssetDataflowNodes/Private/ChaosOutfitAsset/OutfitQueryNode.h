// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "ChaosOutfitAsset/Outfit.h"
#include "OutfitQueryNode.generated.h"


class UDynamicMesh;
class UMaterialInterface;

/**
 * Query an Outfit about its properties.
 */
USTRUCT(Meta = (DataflowOutfit))
struct FChaosOutfitAssetOutfitQueryNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosOutfitAssetOutfitQueryNode, "OutfitQuery", "Outfit", "Outfit Query")

public:

	FChaosOutfitAssetOutfitQueryNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

private:

	/** Input/output collection (output is always a passthrough). */
	UPROPERTY(Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Outfit"))
	TObjectPtr<const UChaosOutfit> Outfit;

	/** Whether this Outfit has any valid pieces. */
	UPROPERTY(Meta = (DataflowOutput))
	bool bHasAnyValidPieces = false;

	/** Whether this Outfit has any valid body sizes. */
	UPROPERTY(Meta = (DataflowOutput))
	bool bHasAnyValidBodySizes = false;

	/** Check that body parts are present in the asset registry when checking for valid body sizes. */
	UPROPERTY(EditAnywhere, Category = "Outfit Query")
	bool bBodyPartMustExist = false;
	
	/** Check that measurements are valid when checking for valid body sizes. */
	UPROPERTY(EditAnywhere, Category = "Outfit Query")
	bool bMeasurementsMustExist = true;
	
	/** Check that some interpolation data exists when checking for valid body sizes. */
	UPROPERTY(EditAnywhere, Category = "Outfit Query")
	bool bInterpolationDataMustExist = true;
};
