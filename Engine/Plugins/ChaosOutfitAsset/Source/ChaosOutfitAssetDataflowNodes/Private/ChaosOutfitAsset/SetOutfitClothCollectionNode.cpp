// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosOutfitAsset/SetOutfitClothCollectionNode.h"
#include "ChaosOutfitAsset/Outfit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SetOutfitClothCollectionNode)

#define LOCTEXT_NAMESPACE "ChaosGetOutfitClothCollectionsNode"

FChaosSetOutfitClothCollectionNode::FChaosSetOutfitClothCollectionNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Outfit);
	RegisterInputConnection(&ClothCollection);
	RegisterInputConnection(&PieceIndex);
	RegisterInputConnection(&LODIndex);
	RegisterOutputConnection(&Outfit, &Outfit);
}

void FChaosSetOutfitClothCollectionNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Outfit))
	{
		if (const TObjectPtr<const UChaosOutfit>& InOutfit = GetValue(Context, &Outfit))
		{
			const int32 InPieceIndex = GetValue(Context, &PieceIndex);
			const int32 InLODIndex = GetValue(Context, &LODIndex);
			if (InOutfit->GetPieces().IsValidIndex(InPieceIndex))
			{

				if (InOutfit->GetPieces()[InPieceIndex].Collections.IsValidIndex(InLODIndex))

				{
					const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &ClothCollection);

					TObjectPtr<UChaosOutfit> OutOutfit = NewObject<UChaosOutfit>();
					OutOutfit->Append(*InOutfit);

					OutOutfit->GetPieces()[InPieceIndex].Collections[InLODIndex] = MakeShared<const FManagedArrayCollection>(InCollection);
					SetValue<TObjectPtr<const UChaosOutfit>>(Context, OutOutfit, &Outfit);
					return;
				}
				else
				{
					Context.Warning(FString::Printf(TEXT("The given LODIndex [%d] doesn't index into this Piece's LOD array of size [%d]"), InLODIndex, InOutfit->GetPieces()[InPieceIndex].Collections.Num()), this, Out);
				}
			}
			else
			{
				Context.Warning(FString::Printf(TEXT("The given PieceIndex [%d] doesn't index into Pieces array of size [%d]"), InPieceIndex, InOutfit->GetPieces().Num()), this, Out);
			}
		}
		SafeForwardInput(Context, &Outfit, &Outfit);
	}
}

#undef LOCTEXT_NAMESPACE
