// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "SetOutfitClothCollectionNode.generated.h"

class UChaosOutfit;

/**
 * Replace the ClothCollection in an Outfit with a new one.
 * Any data derived from the ClothCollection (e.g., Simulation Model, Render Data) will NOT be regenerated in the Outfit.
 */
USTRUCT(Meta = (Experimental, DataflowOutfit))
struct FChaosSetOutfitClothCollectionNode final : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosSetOutfitClothCollectionNode, "SetOutfitClothCollection", "Outfit", "Outfit Cloth Collections")

public:
	FChaosSetOutfitClothCollectionNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	/** The Outfit to be edited. */
	UPROPERTY(Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Outfit"))
	TObjectPtr<const UChaosOutfit> Outfit;

	/** The replacement cloth collection. */
	UPROPERTY(Meta = (DataflowInput))
	FManagedArrayCollection ClothCollection;

	/** The Outfit Piece to replace. */
	UPROPERTY(EditAnywhere, Category = "Set Cloth Collection", Meta =(ClampMin = 0, DataflowInput))
	int32 PieceIndex = 0;

	/** The Outfit LOD to replace. */
	UPROPERTY(EditAnywhere, Category = "Set Cloth Collection", Meta = (ClampMin = 0, DataflowInput))
	int32 LODIndex = 0;
};
