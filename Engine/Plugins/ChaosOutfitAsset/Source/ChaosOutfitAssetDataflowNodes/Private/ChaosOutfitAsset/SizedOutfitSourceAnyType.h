// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowAnyType.h"
#include "Dataflow/DataflowTypePolicy.h"
#include "ChaosOutfitAsset/SizedOutfitSource.h"
#include "Misc/TVariant.h"

#include "SizedOutfitSourceAnyType.generated.h"

USTRUCT()
struct FChaosSizedOutfitSourceOrArrayType
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FChaosSizedOutfitSource> Array;

	FChaosSizedOutfitSourceOrArrayType() = default;

	FChaosSizedOutfitSourceOrArrayType(const FChaosSizedOutfitSource& SizedOutfitSource)
		: Array(&SizedOutfitSource, 1)
	{}

	FChaosSizedOutfitSourceOrArrayType(const TArray<FChaosSizedOutfitSource>& SizedOutfitSources)
		: Array(SizedOutfitSources)
	{}
};

UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FChaosSizedOutfitSource)
UE_DATAFLOW_POLICY_DECLARE_TYPENAME(FChaosSizedOutfitSourceOrArrayType)

/** FChaosSizedOutfitSource or array types. */
USTRUCT()
struct FChaosSizedOutfitSourceOrArrayAnyType : public FDataflowAnyType
{
	GENERATED_BODY()

	struct FPolicy :
		public TDataflowMultiTypePolicy<
			FChaosSizedOutfitSource,
			TArray<FChaosSizedOutfitSource>>
	{};

	using FPolicyType = FPolicy;
	using FStorageType = FChaosSizedOutfitSourceOrArrayType;

	UPROPERTY(EditAnywhere, Category = Value)
	FChaosSizedOutfitSourceOrArrayType Value;
};
