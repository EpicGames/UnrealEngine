// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ChaosOutfitAssetEditor : ModuleRules
{
	public ChaosOutfitAssetEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetDefinition",
				"BaseCharacterFXEditor",
				"ChaosClothAsset",
				"ChaosClothAssetEngine",
				"ChaosClothAssetTools",
				"ChaosOutfitAssetEngine",
				"ContentBrowser",
				"Core",
				"CoreUObject",
				"DataflowCore",
				"DataflowEditor",
				"DataflowEngine",
				"Engine",
				"GeometryCore",
				"GeometryFramework",
				"InputCore",
				"InteractiveToolsFramework",
				"Projects",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"ToolWidgets",
				"UnrealEd"
			}
		);

		PrivateIncludePaths.Add(System.IO.Path.Combine(GetModuleDirectory("ChaosClothAssetTools"), "Internal"));  // For thumbnail renderer
	}
}
