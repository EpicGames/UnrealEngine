// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosOutfitAsset/AssetDefinition_OutfitAsset.h"
#include "ChaosOutfitAsset/OutfitAsset.h"
#include "Dataflow/DataflowEditor.h"
#include "Dataflow/DataflowEditorToolkit.h"
#include "Dataflow/DataflowSimulationScene.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "Toolkits/SimpleAssetEditor.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_OutfitAsset"

namespace UE::Chaos::OutfitAsset::Private
{
	static const FLinearColor OutfitAssetColor = FColor(243, 244, 11);  // Bright yellowx
}

FText UAssetDefinition_OutfitAsset::GetAssetDisplayName() const
{
	return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_OutfitAsset", "OutfitAsset");
}

TSoftClassPtr<UObject> UAssetDefinition_OutfitAsset::GetAssetClass() const
{
	return UChaosOutfitAsset::StaticClass();
}

FLinearColor UAssetDefinition_OutfitAsset::GetAssetColor() const
{
	return UE::Chaos::OutfitAsset::Private::OutfitAssetColor;
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_OutfitAsset::GetAssetCategories() const
{
	static const TArray<FAssetCategoryPath> Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Physics, LOCTEXT("AssetDefinition_OutfitAssetSubMenu", "Chaos Cloth"), ECategoryMenuType::Section)
		};
	return Categories;
}

UThumbnailInfo* UAssetDefinition_OutfitAsset::LoadThumbnailInfo(const FAssetData& InAsset) const
{
	return UE::Editor::FindOrCreateThumbnailInfo(InAsset.GetAsset(), USceneThumbnailInfo::StaticClass());
}

EAssetCommandResult UAssetDefinition_OutfitAsset::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	TArray<UChaosOutfitAsset*> OutfitObjects = OpenArgs.LoadObjects<UChaosOutfitAsset>();

	// For now the Dataflow editor only works on one asset at a time
	if (OutfitObjects.Num() > 0 && OutfitObjects[0])
	{
		if (OutfitObjects[0]->GetDataflowInstance().GetDataflowAsset())
		{
			// Has a Dataflow Asset, open in the Dataflow editor
			UAssetEditorSubsystem* const AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
			if (UDataflowEditor* const AssetEditor = NewObject<UDataflowEditor>(AssetEditorSubsystem, NAME_None, RF_Transient))
			{
				AssetEditor->RegisterToolCategories({"General", "Cloth"});

				if (UDataflowSimulationSettings* const SimulationSettings = NewObject<UDataflowSimulationSettings>())
				{
					SimulationSettings->bIsSimulationPlayingByDefault = true;
					SimulationSettings->bIsAsyncCachingSupported = false;
					SimulationSettings->bIsAsyncCachingEnabledByDefault = false;
					AssetEditor->AddEditorSettings(SimulationSettings);
				}

				if (UDataflowEvaluationSettings* const EvaluationSettings = NewObject<UDataflowEvaluationSettings>())
				{
					EvaluationSettings->bAllowEvaluationInPIE = true;
					AssetEditor->AddEditorSettings(EvaluationSettings);
				}

				static const TCHAR* const Path = TEXT("/ChaosOutfitAsset/BP_OutfitPreview.BP_OutfitPreview_C");
				const TSubclassOf<AActor> PreviewClass = StaticLoadClass(AActor::StaticClass(), nullptr, Path);
				AssetEditor->Initialize({ OutfitObjects[0] }, PreviewClass);
			}
		}
		else
		{
			FSimpleAssetEditor::CreateEditor(EToolkitMode::Standalone, OpenArgs.ToolkitHost, OutfitObjects[0]);
		}
		return EAssetCommandResult::Handled;
	}
	return EAssetCommandResult::Unhandled;
}

#undef LOCTEXT_NAMESPACE
