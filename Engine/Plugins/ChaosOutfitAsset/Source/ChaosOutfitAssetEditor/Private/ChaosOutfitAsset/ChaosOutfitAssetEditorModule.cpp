// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosOutfitAsset/AssetDefinition_OutfitAsset.h"
#include "Algo/AllOf.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ChaosClothAsset/ClothAssetThumbnailRenderer.h"
#include "ChaosOutfitAsset/OutfitAsset.h"
#include "ChaosOutfitAsset/OutfitAssetEditorStyle.h"
#include "ChaosOutfitAsset/OutfitEditorCommands.h"
#include "ChaosOutfitAsset/OutfitRenderableTypes.h"
#include "Dataflow/AssetDefinition_DataflowAsset.h"
#include "Dataflow/DataflowTemplateRegistry.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/Skeleton.h"
#include "Factories/SkeletonFactory.h"
#include "Framework/Commands/Commands.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "AssetToolsModule.h"
#include "ContentBrowserMenuContexts.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "OutfitAssetEditorModule"

namespace UE::Chaos::OutfitAsset
{
	class FOutfitAssetEditorModule : public IModuleInterface
	{
	public:
		//~ Begin IModuleInterface interface
		virtual void StartupModule() override
		{
			// Register asset icons
			FOutfitAssetEditorStyle::Get();

			// Register editor commands
			FOutfitEditorCommands::Register();

			// Register the asset menus
			StartupCallbackDelegateHandle = UToolMenus::RegisterStartupCallback(
				FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FOutfitAssetEditorModule::RegisterMenus));

			UThumbnailManager::Get().RegisterCustomRenderer(UChaosOutfitAsset::StaticClass(), UChaosClothAssetThumbnailRenderer_Internal::StaticClass());

			const FSlateIcon ChaosOutfitAssetIcon("OutfitAssetEditorStyle", "ClassThumbnail.ChaosOutfitAsset");

			FDataflowTemplateRegistry::Get().RegisterTemplateAsset(
				UChaosOutfitAsset::StaticClass(),
				{
					.AssetPath = FSoftObjectPath(TEXT("/ChaosOutfitAsset/EmptyOutfitAssetTemplate.EmptyOutfitAssetTemplate")),
					.DisplayName = LOCTEXT("SelectEmptyTemplate", "Empty Dataflow"),
					.Tooltip = LOCTEXT("SelectEmptyTemplateTooltip", "Add an empty Dataflow with an Outfit Asset Terminal node."),
					 .Icon = ChaosOutfitAssetIcon
				});

			FDataflowTemplateRegistry::Get().RegisterTemplateAsset(
				UChaosOutfitAsset::StaticClass(),
				{
					.AssetPath = FSoftObjectPath(TEXT("/ChaosOutfitAsset/OutfitAssetTemplate.OutfitAssetTemplate")),
					.DisplayName = LOCTEXT("SelectSimpleOutfitTemplate", "Simple Outfit"),
					.Tooltip = LOCTEXT("SelectSimpleOutfitTemplateTooltip", "Add a Dataflow with a simple Cloth Asset aggregator graph. Allows to simulate multiple Cloth Assets from the same ChaosClothComponent."),
					 .Icon = ChaosOutfitAssetIcon
				});

			FDataflowTemplateRegistry::Get().RegisterTemplateAsset(
				UChaosOutfitAsset::StaticClass(),
				{
					.AssetPath = FSoftObjectPath(TEXT("/ChaosOutfitAsset/MakeResizableOutfitTemplate.MakeResizableOutfitTemplate")),
					.DisplayName = LOCTEXT("SelectResizableOutfitTemplate", "Resizable Outfit"),
					.Tooltip = LOCTEXT("SelectResizableOutfitTemplateTooltip", "Add a Dataflow that builds a single resizable garment from multiple Cloth Assets. Body sizes will have to be provided in addition to the multiple Cloth Assets."),
					 .Icon = ChaosOutfitAssetIcon
				});

			FDataflowTemplateRegistry::Get().RegisterTemplateAsset(
				UChaosOutfitAsset::StaticClass(),
				{
					.AssetPath = FSoftObjectPath(TEXT("/ChaosOutfitAsset/ResizeOutfitTemplate.ResizeOutfitTemplate")),
					.DisplayName = LOCTEXT("SelectResizingGraph", "Resizing Graph"),
					.Tooltip = LOCTEXT("SelectResizingGraphTooltip", "Resizing graph to test a resizable Outfit Asset."),
					 .Icon = ChaosOutfitAssetIcon
				});

			DataflowMenusHandle = UE::DataflowAssetDefinitionHelpers::RegisterDataflowAssetMenus(UChaosOutfitAsset::StaticClass());

			// Register visualizations
			RegisterOutfitRenderableTypes();
		}

		virtual void ShutdownModule() override
		{
			if (UObjectInitialized())
			{
				// Note: Renderable types are static instances, therefore no deregistration is needed

				UE::DataflowAssetDefinitionHelpers::UnregisterDataflowAssetMenus(DataflowMenusHandle);

				UToolMenus::UnRegisterStartupCallback(StartupCallbackDelegateHandle);
				FOutfitEditorCommands::Unregister();

				UThumbnailManager::Get().UnregisterCustomRenderer(UChaosOutfitAsset::StaticClass());
			}
		}
		//~ End IModuleInterface interface

private:
		void RegisterMenus()
		{
			FToolMenuOwnerScoped OwnerScoped(this);  // Allows cleanup when module unloads.

			UToolMenu* const ToolMenu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu.ChaosOutfitAsset");
			FToolMenuSection& Section = ToolMenu->FindOrAddSection("GetAssetActions");

			Section.AddDynamicEntry("ConvertToSkeletalMesh", FNewToolMenuSectionDelegate::CreateLambda(
				[this](FToolMenuSection& Section)
				{
					if (const UContentBrowserAssetContextMenuContext* const Context = Section.FindContext<UContentBrowserAssetContextMenuContext>())
					{
						if (Algo::AllOf(
							Context->SelectedAssets,  // Don't use Context->GetSelectedObjects() to avoid unnecessary loading the asset
							[](const FAssetData& Asset)
							{
								return Asset.AssetClassPath == UChaosOutfitAsset::StaticClass()->GetClassPathName();
							}))
						{
							const bool bIsChaosClothAssetToolsModuleLoaded = FModuleManager::Get().IsModuleLoaded("ChaosClothAssetTools");

							const TSharedPtr<FUICommandList> CommandList = MakeShared<FUICommandList>();
							CommandList->MapAction(
								FOutfitEditorCommands::Get().GetConvertToSkeletalMesh(),
								FExecuteAction::CreateWeakLambda(
									Context, 
									[Context]()  // ExecuteAction
									{
										const float NumSteps = (float)Context->SelectedAssets.Num() + 1.f;
										FScopedSlowTask SlowTask(NumSteps, LOCTEXT("ConvertingToSkeletalMeshes", "Converting Outfit(s) to SkeletalMesh(es)..."));
										SlowTask.MakeDialog();

										SlowTask.EnterProgressFrame(1.f, LOCTEXT("LoadingOutfitAsset", "Loading OutfitAsset(s)..."));
										const TArray<TObjectPtr<UObject>> SelectedObjects(Context->LoadSelectedObjects<UObject>());

										for (const TObjectPtr<const UObject> SelectedObject : SelectedObjects)
										{
											SlowTask.EnterProgressFrame(1.f, LOCTEXT("ConvertingToSkeletalMesh", "Converting OutfitAsset to SkeletalMesh..."));
											if (const TObjectPtr<const UChaosOutfitAsset> OutfitAsset = Cast<const UChaosOutfitAsset>(SelectedObject))
											{
												ConvertToSkeletalMesh(*OutfitAsset);
											}
										}
									}),
								FCanExecuteAction::CreateWeakLambda(
									Context,
									[bIsChaosClothAssetToolsModuleLoaded]()  // CanExecuteAction
									{
										return bIsChaosClothAssetToolsModuleLoaded;
									}));

							const TAttribute<FText> ToolTipOverride = bIsChaosClothAssetToolsModuleLoaded ?
								LOCTEXT("ConvertToSkeletalMeshes", "Convert the selected OutfitAsset(s) to SkeletalMesh(es).") :
								LOCTEXT("ChaosClothAssetEditorMustBeLoaded", "The ChaosClothAssetEditor plug-in must be loaded to enable this action.");
							Section.AddMenuEntryWithCommandList(
								FOutfitEditorCommands::Get().GetConvertToSkeletalMesh(),
								CommandList,
								TAttribute<FText>(),
								ToolTipOverride,
								FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Convert"));  // Could also use Icons.SkeletalMesh if it makes more sense
						}
					}
				}));
		}

		static void ConvertToSkeletalMesh(const UChaosOutfitAsset& OutfitAsset)
		{
			// Make a unique name from the outfit asset name with the correct recommended asset prefixes
			const FString SkeletalMeshPath = FPackageName::GetLongPackagePath(OutfitAsset.GetOutermost()->GetName());
			const FString OutfitAssetName = OutfitAsset.GetName();
			FString SkeletalMeshName = FString(TEXT("SK_")) + (OutfitAssetName.StartsWith(TEXT("OA_")) ? OutfitAssetName.RightChop(3) : OutfitAssetName);
			FString SkeletalMeshPackageName = FPaths::Combine(SkeletalMeshPath, SkeletalMeshName);
			if (FindPackage(nullptr, *SkeletalMeshPackageName))
			{
				MakeUniqueObjectName(nullptr, UPackage::StaticClass(), FName(SkeletalMeshPackageName)).ToString(SkeletalMeshPackageName);
				SkeletalMeshName = FPaths::GetBaseFilename(SkeletalMeshPackageName);
			}
			UPackage* const SkeletalMeshPackage = CreatePackage(*SkeletalMeshPackageName);
			if (USkeletalMesh* const SkeletalMesh =
				NewObject<USkeletalMesh>(
					SkeletalMeshPackage,
					USkeletalMesh::StaticClass(),
					FName(SkeletalMeshName),
					RF_Public | RF_Standalone | RF_Transactional))
			{
				OutfitAsset.ExportToSkeletalMesh(*SkeletalMesh);

				SkeletalMesh->MarkPackageDirty();

				FAssetRegistryModule::AssetCreated(SkeletalMesh);  // Notify the asset registry

				// Add the skeleton
				FString SkeletonName = FString(TEXT("SKEL_")) + (OutfitAssetName.StartsWith(TEXT("OA_")) ? OutfitAssetName.RightChop(3) : OutfitAssetName);
				FString SkeletonPackageName = FPaths::Combine(SkeletalMeshPath, SkeletonName);
				if (FindPackage(nullptr, *SkeletonPackageName))
				{
					MakeUniqueObjectName(nullptr, UPackage::StaticClass(), FName(SkeletonPackageName)).ToString(SkeletonPackageName);
					SkeletonName = FPaths::GetBaseFilename(SkeletonPackageName);
				}
				USkeletonFactory* const SkeletonFactory = NewObject<USkeletonFactory>();
				SkeletonFactory->TargetSkeletalMesh = SkeletalMesh;

				FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
				AssetToolsModule.Get().CreateAsset(SkeletonName, SkeletalMeshPath, USkeleton::StaticClass(), SkeletonFactory);
			}
		}

		FDelegateHandle StartupCallbackDelegateHandle;
		FDelegateHandle DataflowMenusHandle;
	};
} // namespace UE::Chaos::OutfitAsset

IMPLEMENT_MODULE(UE::Chaos::OutfitAsset::FOutfitAssetEditorModule, ChaosOutfitAssetEditor);

#undef LOCTEXT_NAMESPACE
