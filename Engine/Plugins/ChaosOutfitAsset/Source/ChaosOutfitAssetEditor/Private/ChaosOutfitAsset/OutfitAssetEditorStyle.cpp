// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosOutfitAsset/OutfitAssetEditorStyle.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/LazySingleton.h"
#include "Styling/SlateStyleRegistry.h"

namespace UE::Chaos::OutfitAsset
{
	FOutfitAssetEditorStyle::FOutfitAssetEditorStyle()
		: FSlateStyleSet("OutfitAssetEditorStyle")
	{
		const TSharedPtr<IPlugin> ChaosOutfitAssetPlugin = IPluginManager::Get().FindPlugin("ChaosOutfitAsset");
		if (ChaosOutfitAssetPlugin.IsValid())
		{
			SetContentRoot(ChaosOutfitAssetPlugin->GetBaseDir() / TEXT("Resources"));

			Set("ClassIcon.ChaosOutfitAsset", new FSlateVectorImageBrush(RootToContentDir(TEXT("OutfitAsset_16.svg")), FVector2D(16)));
			Set("ClassThumbnail.ChaosOutfitAsset", new FSlateVectorImageBrush(RootToContentDir(TEXT("OutfitAsset_64.svg")), FVector2D(64)));
		}
		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	FOutfitAssetEditorStyle::~FOutfitAssetEditorStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}

	FOutfitAssetEditorStyle& FOutfitAssetEditorStyle::Get()
	{
		return TLazySingleton<FOutfitAssetEditorStyle>::Get();
	}

	void FOutfitAssetEditorStyle::TearDown()
	{
		TLazySingleton<FOutfitAssetEditorStyle>::TearDown();
	}
} // namespace UE::Chaos::OutfitAsset
