// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"
#include "Misc/LazySingleton.h"

namespace UE::Chaos::OutfitAsset
{
	/**
	 * Editor style setting up the outfit asset icons in editor.
	 */
	class FOutfitAssetEditorStyle final : public FSlateStyleSet
	{
	public:
		static FOutfitAssetEditorStyle& Get();
		static void TearDown();

	private:
		friend class ::FLazySingleton;

		FOutfitAssetEditorStyle();
		virtual ~FOutfitAssetEditorStyle() override;
	};
}  // namespace UE::Chaos::OutfitAsset
