// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosOutfitAsset/OutfitAssetFactory.h"
#include "ChaosOutfitAsset/OutfitAsset.h"
#include "Dataflow/AssetDefinition_DataflowAsset.h"
#include "Dialog/SMessageDialog.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OutfitAssetFactory)

#define LOCTEXT_NAMESPACE "OutfitAssetFactory"

DEFINE_LOG_CATEGORY_STATIC(LogChaosOutfitAssetFactory, Log, All);

UChaosOutfitAssetFactory::UChaosOutfitAssetFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bEditorImport = true;
	bEditAfterNew = true;
	SupportedClass = UChaosOutfitAsset::StaticClass();
}

UObject* UChaosOutfitAssetFactory::FactoryCreateNew(UClass* Class, UObject* Parent, FName Name, EObjectFlags Flags, UObject* /*Context*/, FFeedbackContext* /*Warn*/)
{
	UObject* const NewAsset = NewObject<UObject>(Parent, Class, Name, Flags | RF_Transactional | RF_Public | RF_Standalone);
	if (!NewAsset)
	{
		const FText ErrorMessage = FText::Format(LOCTEXT("ErrorCreatingNewAsset", "Failed to create a new asset {0}."), FText::FromName(Name));
		UE_LOGF(LogChaosOutfitAssetFactory, Error, "%ls", *ErrorMessage.ToString());
		return nullptr;
	}
	NewAsset->MarkPackageDirty();

	// No default blank option since cloth templates already provide one
	UE::DataflowAssetDefinitionHelpers::SetDataflowFromTemplatePicker(NewAsset, /*bShowDefaultBlankOption*/ false);
	return NewAsset;
}

FString UChaosOutfitAssetFactory::GetDefaultNewAssetName() const
{
	return FString(TEXT("OA_NewOutfitAsset"));
}

#undef LOCTEXT_NAMESPACE
