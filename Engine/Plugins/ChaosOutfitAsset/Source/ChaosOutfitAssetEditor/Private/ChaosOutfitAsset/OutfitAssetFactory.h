// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "OutfitAssetFactory.generated.h"

/**
 * Allows the Outfit Asset to be created from the Editor's menus.
 */
UCLASS()
class UChaosOutfitAssetFactory final : public UFactory
{
	GENERATED_BODY()
public:
	UChaosOutfitAssetFactory(const FObjectInitializer& ObjectInitializer);

private:
	//~ Begin UFactory interface
	virtual bool CanCreateNew() const override { return true; }
	virtual bool FactoryCanImport(const FString& Filename) override { return false; }
	virtual bool ShouldShowInNewMenu() const override { return true; }
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual FString GetDefaultNewAssetName() const override;
	//~ End UFactory interface
};
