// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosOutfitAsset/OutfitEditorCommands.h"
#include "ChaosOutfitAsset/OutfitEditorStyle.h"

#define LOCTEXT_NAMESPACE "ChaosClothAssetOutfitEditorCommands"

namespace UE::Chaos::OutfitAsset
{
	FOutfitEditorCommands::FOutfitEditorCommands()
		: TBaseCharacterFXEditorCommands<FOutfitEditorCommands>(
			"ChaosClothOutfitEditor",
			LOCTEXT("ContextDescription", "Outfit Editor"), 
			NAME_None, // Parent
			FOutfitEditorStyle::Get().GetStyleSetName())
	{
	}

	void FOutfitEditorCommands::RegisterCommands()
	{
		TBaseCharacterFXEditorCommands::RegisterCommands();

		UI_COMMAND(ConvertToSkeletalMesh, "Convert to SkeletalMesh", "Convert the selected OutfitAsset(s) to SkeletalMesh(es).", EUserInterfaceActionType::Button, FInputChord());
	}

	void FOutfitEditorCommands::UpdateToolCommandBinding(UInteractiveTool* Tool, TSharedPtr<FUICommandList> UICommandList, bool bUnbind)
	{
		if (FOutfitEditorCommands::IsRegistered())
		{
			if (bUnbind)
			{
				FOutfitEditorCommands::Get().UnbindActiveCommands(UICommandList);
			}
			else
			{
				FOutfitEditorCommands::Get().BindCommandsForCurrentTool(UICommandList, Tool);
			}
		}
	}
} // namespace UE::Chaos::OutfitAsset

#undef LOCTEXT_NAMESPACE
