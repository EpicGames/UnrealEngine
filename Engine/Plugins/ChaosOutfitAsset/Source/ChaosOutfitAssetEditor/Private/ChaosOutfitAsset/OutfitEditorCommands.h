// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseCharacterFXEditorCommands.h"

namespace UE::Chaos::OutfitAsset
{
	class FOutfitEditorCommands final : public TBaseCharacterFXEditorCommands<FOutfitEditorCommands>
	{
	public:
		FOutfitEditorCommands();

		/**
		 * Add or remove commands relevant to Tool to the given UICommandList.
		 * Call this when the active tool changes (e.g. on ToolManager.OnToolStarted / OnToolEnded).
		 * @param bUnbind if true, commands are removed, otherwise added.
		 */
		static void UpdateToolCommandBinding(UInteractiveTool* Tool, TSharedPtr<FUICommandList> UICommandList, bool bUnbind = false);

		const TSharedPtr<FUICommandInfo>& GetConvertToSkeletalMesh() const
		{
			return ConvertToSkeletalMesh;
		}

	private:
		friend TCommands<FOutfitEditorCommands>;

		//~ Begin TBaseCharacterFXEditorCommands<FOutfitEditorCommands> interface
		virtual void RegisterCommands() override;

		virtual void GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs) override {}
		//~ End TBaseCharacterFXEditorCommands<FOutfitEditorCommands> interface

		TSharedPtr<FUICommandInfo> ConvertToSkeletalMesh;
	};
} // namespace UE::Chaos::OutfitAsset
