// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosOutfitAsset/OutfitEditorStyle.h"
#include "Misc/LazySingleton.h"
#include "Styling/SlateStyleRegistry.h"

namespace UE::Chaos::OutfitAsset
{
	FOutfitEditorStyle::FOutfitEditorStyle()
		: FSlateStyleSet(StyleName)
	{
		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	FOutfitEditorStyle::~FOutfitEditorStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}

	FOutfitEditorStyle& FOutfitEditorStyle::Get()
	{
		return TLazySingleton<FOutfitEditorStyle>::Get();
	}

	void FOutfitEditorStyle::TearDown()
	{
		TLazySingleton<FOutfitEditorStyle>::TearDown();
	}
} // namespace UE::Chaos::OutfitAsset
