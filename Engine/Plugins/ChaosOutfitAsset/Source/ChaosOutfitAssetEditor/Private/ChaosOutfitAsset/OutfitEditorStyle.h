// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"
#include "Misc/LazySingleton.h"

namespace UE::Chaos::OutfitAsset
{
	/**
	 * Slate style set for Cloth Editor.
	 */
	class CHAOSOUTFITASSETEDITOR_API FOutfitEditorStyle final : public FSlateStyleSet
	{
	public:
		static inline const FName StyleName = "OutfitEditorStyle";

		static FOutfitEditorStyle& Get();
		static void TearDown();

	private:
		friend class ::FLazySingleton;

		FOutfitEditorStyle();
		virtual ~FOutfitEditorStyle() override;
	};
} // namespace UE::Chaos::OutfitAsset
