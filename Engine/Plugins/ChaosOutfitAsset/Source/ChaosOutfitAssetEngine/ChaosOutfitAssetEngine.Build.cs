// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ChaosOutfitAssetEngine : ModuleRules
{
	public ChaosOutfitAssetEngine(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"ChaosClothAssetEngine",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Chaos",
				"ChaosClothAsset",
				"DataflowEngine",
				"Engine",
				"MeshResizingCore",
				"RenderCore",
				"RHI",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[]
			{
				"DerivedDataCache",
			}
		);
	}
}
