// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "ChaosOutfitAsset/OutfitAssetPrivate.h"

#define LOCTEXT_NAMESPACE "OutfitAssetEngineModule"

DEFINE_LOG_CATEGORY(LogChaosOutfitAsset);

namespace UE::Chaos::OutfitAsset
{
	class FOutfitAssetEngineModule : public IModuleInterface
	{
	public:
		virtual void StartupModule() override
		{
		}

		virtual void ShutdownModule() override
		{
		}
	};
}  // namespace UE::Chaos::OutfitAsset

IMPLEMENT_MODULE(UE::Chaos::OutfitAsset::FOutfitAssetEngineModule, ChaosOutfitAssetEngine);

#undef LOCTEXT_NAMESPACE
