// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosOutfitAsset/CollectionOutfitFacade.h"
#include "Algo/AllOf.h"
#include "Algo/AnyOf.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ChaosClothAsset/ClothAssetBase.h"
#include "ChaosOutfitAsset/BodyUserData.h"
#include "ChaosOutfitAsset/OutfitAssetPrivate.h"
#include "ChaosOutfitAsset/OutfitCollection.h"
#include "Engine/SkeletalMesh.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Serialization/Archive.h"
#include "UObject/SoftObjectPath.h"

#define LOCTEXT_NAMESPACE "CollectionOutfitFacade"

namespace UE::Chaos::OutfitAsset
{
	namespace Private
	{
		// Utility class to transfer const access from the pointer to the pointed data
		template<typename T>
		class TConstPreservingPointer
		{
		public:
			TConstPreservingPointer(T* const InPtr = nullptr) : Ptr(InPtr) {}
			TConstPreservingPointer(const TConstPreservingPointer&) = delete;  // Non copyable as to preserve the source const access
			TConstPreservingPointer& operator=(const TConstPreservingPointer&) = delete;
			TConstPreservingPointer& operator=(T* const InPtr)
			{
				Ptr = InPtr;
				return *this;
			}
			bool IsValid() const
			{
				return Ptr != nullptr;
			}
			operator bool() const
			{
				return IsValid();
			}
			const T* operator->() const
			{
				return Ptr;
			}
			const T& operator*() const
			{
				return *Ptr;
			}
			T* operator->()
			{
				return Ptr;
			}
			T& operator*()
			{
				check(IsValid());
				return *Ptr;
			}
		private:
			T* Ptr = nullptr;
		};

		template<typename T>
		using TManagedArrayPtr = TConstPreservingPointer<TManagedArray<T>>;

		// Outfit collection schema implementation
		struct FOutfitCollection
		{
			FManagedArrayCollection& ManagedArrayCollection;

			struct
			{
				TManagedArrayPtr<FGuid> Guid;  // Outfit guid to distinguish between separated sized outfit merged into a single outfit
				TManagedArrayPtr<int32> BodySize;  // Body size index in the BodySizes table
				TManagedArrayPtr<FString> Name;  // Name of the source asset making this outfit size
				TManagedArrayPtr<int32> PiecesStart;  // Outfit piece start index in the Pieces table
				TManagedArrayPtr<int32> PiecesCount;  // Number of outfit pieces for this outfit
			} Outfits;

			struct
			{
				TManagedArrayPtr<FGuid> Guid;  // Outfit Piece GUID (same as FChaosOutfitPiece::Guid), in case the name is duplicated
				TManagedArrayPtr<FString> Name;  // Outfit Piece Name (for debugging, same as FChaosOutfitPiece::Name)
			} Pieces;

			struct
			{
				TManagedArrayPtr<FString> Name;  // Name of this body size
				TManagedArrayPtr<int32> BodyPartsStart;  // Body part start index in the BodyParts table
				TManagedArrayPtr<int32> BodyPartsCount;  // Number of body parts for this body size
			} BodySizes;

			struct
			{
				TManagedArrayPtr<FSoftObjectPath> SkeletalMeshPath;  // Body parts skeletal mesh (several per sizes, e.g. body, head)
				TManagedArrayPtr<TArray<int32>> RBFInterpolationSampleIndices; // Precomputed RBF interpolation sample indices
				TManagedArrayPtr<TArray<FVector3f>> RBFInterpolationSampleRestPositions; // Precomputed RBF interpolation sample rest positions
				TManagedArrayPtr<TArray<float>> RBFInterpolationWeights; // Precomputed RBF interpolation weights
			} BodyParts;

			struct
			{
				TManagedArrayPtr<FString> Name;  // Measurements name (several per sizes, e.g. one for Height, Waist, ...etc.)
			} Measurements;

			explicit FOutfitCollection(FManagedArrayCollection& InManagedArrayCollection)
				: ManagedArrayCollection(InManagedArrayCollection)
			{
				using namespace OutfitCollection;
				Outfits.Guid = ManagedArrayCollection.FindAttribute<FGuid>(Attribute::Outfits::Guid, Group::Outfits);
				Outfits.BodySize = ManagedArrayCollection.FindAttribute<int32>(Attribute::Outfits::BodySize, Group::Outfits);
				Outfits.Name = ManagedArrayCollection.FindAttribute<FString>(Attribute::Outfits::Name, Group::Outfits);
				Outfits.PiecesStart = ManagedArrayCollection.FindAttribute<int32>(Attribute::Outfits::PiecesStart, Group::Outfits);
				Outfits.PiecesCount = ManagedArrayCollection.FindAttribute<int32>(Attribute::Outfits::PiecesCount, Group::Outfits);
				Pieces.Guid = ManagedArrayCollection.FindAttribute<FGuid>(Attribute::Pieces::Guid, Group::Pieces);
				Pieces.Name = ManagedArrayCollection.FindAttribute<FString>(Attribute::Pieces::Name, Group::Pieces);
				BodySizes.Name = ManagedArrayCollection.FindAttribute<FString>(Attribute::BodySizes::Name, Group::BodySizes);
				BodySizes.BodyPartsStart = ManagedArrayCollection.FindAttribute<int32>(Attribute::BodySizes::BodyPartsStart, Group::BodySizes);
				BodySizes.BodyPartsCount = ManagedArrayCollection.FindAttribute<int32>(Attribute::BodySizes::BodyPartsCount, Group::BodySizes);
				BodyParts.SkeletalMeshPath = ManagedArrayCollection.FindAttribute<FSoftObjectPath>(Attribute::BodyParts::SkeletalMeshPath, Group::BodyParts);
				BodyParts.RBFInterpolationSampleIndices = ManagedArrayCollection.FindAttribute<TArray<int32>>(Attribute::BodyParts::RBFInterpolationSampleIndices, Group::BodyParts);
				BodyParts.RBFInterpolationSampleRestPositions = ManagedArrayCollection.FindAttribute<TArray<FVector3f>>(Attribute::BodyParts::RBFInterpolationSampleRestPositions, Group::BodyParts);
				BodyParts.RBFInterpolationWeights = ManagedArrayCollection.FindAttribute<TArray<float>>(Attribute::BodyParts::RBFInterpolationWeights, Group::BodyParts);
				Measurements.Name = ManagedArrayCollection.FindAttribute<FString>(Attribute::Measurements::Name, Group::Measurements);
			}

			bool IsValid() const
			{
				using namespace OutfitCollection;
				return
					Outfits.Guid &&
					Outfits.BodySize &&
					Outfits.Name &&
					Outfits.PiecesStart &&
					Outfits.PiecesCount &&
					Pieces.Guid &&
					Pieces.Name &&
					BodySizes.Name &&
					BodySizes.BodyPartsStart &&
					BodySizes.BodyPartsCount &&
					BodyParts.SkeletalMeshPath &&
					BodyParts.RBFInterpolationSampleIndices &&
					BodyParts.RBFInterpolationSampleRestPositions &&
					BodyParts.RBFInterpolationWeights &&
					Measurements.Name;
			}

			void DefineSchema()
			{
				if (!IsValid())
				{
					using namespace OutfitCollection;
					Outfits.Guid = &ManagedArrayCollection.AddAttribute<FGuid>(Attribute::Outfits::Guid, Group::Outfits);
					Outfits.BodySize = &ManagedArrayCollection.AddAttribute<int32>(Attribute::Outfits::BodySize, Group::Outfits, Group::BodySizes);
					Outfits.Name = &ManagedArrayCollection.AddAttribute<FString>(Attribute::Outfits::Name, Group::Outfits);
					Outfits.PiecesStart = &ManagedArrayCollection.AddAttribute<int32>(Attribute::Outfits::PiecesStart, Group::Outfits, Group::Pieces);
					Outfits.PiecesCount = &ManagedArrayCollection.AddAttribute<int32>(Attribute::Outfits::PiecesCount, Group::Outfits);
					Pieces.Guid = &ManagedArrayCollection.AddAttribute<FGuid>(Attribute::Pieces::Guid, Group::Pieces);
					Pieces.Name = &ManagedArrayCollection.AddAttribute<FString>(Attribute::Pieces::Name, Group::Pieces);
					BodySizes.Name = &ManagedArrayCollection.AddAttribute<FString>(Attribute::BodySizes::Name, Group::BodySizes);
					BodySizes.BodyPartsStart = &ManagedArrayCollection.AddAttribute<int32>(Attribute::BodySizes::BodyPartsStart, Group::BodySizes, Group::BodyParts);
					BodySizes.BodyPartsCount = &ManagedArrayCollection.AddAttribute<int32>(Attribute::BodySizes::BodyPartsCount, Group::BodySizes);
					BodyParts.SkeletalMeshPath = &ManagedArrayCollection.AddAttribute<FSoftObjectPath>(Attribute::BodyParts::SkeletalMeshPath, Group::BodyParts);
					BodyParts.RBFInterpolationSampleIndices = &ManagedArrayCollection.AddAttribute<TArray<int32>>(Attribute::BodyParts::RBFInterpolationSampleIndices, Group::BodyParts);
					BodyParts.RBFInterpolationSampleRestPositions = &ManagedArrayCollection.AddAttribute<TArray<FVector3f>>(Attribute::BodyParts::RBFInterpolationSampleRestPositions, Group::BodyParts);
					BodyParts.RBFInterpolationWeights = &ManagedArrayCollection.AddAttribute<TArray<float>>(Attribute::BodyParts::RBFInterpolationWeights, Group::BodyParts);
					Measurements.Name = &ManagedArrayCollection.AddAttribute<FString>(Attribute::Measurements::Name, Group::Measurements);
				}
			}

			void Reset()
			{
				using namespace OutfitCollection;
				ManagedArrayCollection.EmptyGroup(Group::Outfits);
				ManagedArrayCollection.EmptyGroup(Group::Pieces);
				ManagedArrayCollection.EmptyGroup(Group::BodySizes);
				ManagedArrayCollection.EmptyGroup(Group::BodyParts);
				ManagedArrayCollection.EmptyGroup(Group::Measurements);
			}

			void PostSerialize(const FArchive& Ar)
			{
				if (Ar.IsLoading())
				{
					auto ConvertStringToSoftObjectPath = [](const TManagedArray<FString>& StringArray, TManagedArray<FSoftObjectPath>& SoftObjectPathArray)
						{
							check(StringArray.Num() == SoftObjectPathArray.Num());
							for (int32 Index = 0; Index < StringArray.Num(); ++Index)
							{
								SoftObjectPathArray[Index] = FSoftObjectPath(StringArray[Index]);
								SoftObjectPathArray[Index].PreSavePath(); // This will fixup the path with redirects.
							}
						};

					// BodyParts::SkeletalMesh -> BodyParts::SkeletalMeshPath
					using namespace OutfitCollection;
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					if (TManagedArrayPtr<FString> SkeletalMeshes = ManagedArrayCollection.FindAttribute<FString>(Attribute::BodyParts::SkeletalMesh, Group::BodyParts))
					{
						if (!BodyParts.SkeletalMeshPath)
						{
							BodyParts.SkeletalMeshPath = &ManagedArrayCollection.AddAttribute<FSoftObjectPath>(Attribute::BodyParts::SkeletalMeshPath, Group::BodyParts);
						}
						ConvertStringToSoftObjectPath(*SkeletalMeshes, *BodyParts.SkeletalMeshPath);
						ManagedArrayCollection.RemoveAttribute(Attribute::BodyParts::SkeletalMesh, Group::BodyParts);
					}
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}
			}
		};

		class FBodyMatchParameters
		{
		public:
			enum class EBodyPlacement : uint8 { Top, Bottom, Full };

			explicit FBodyMatchParameters(const TMap<FString, float>& Measurements)
			{
				Hip = Measurements.FindRef(TEXT("Hip"), InvalidMeasurement);
				float Bust = Measurements.FindRef(TEXT("Bust"), InvalidMeasurement);
				if (Bust == InvalidMeasurement)
				{
					Bust = Measurements.FindRef(TEXT("Chest"), InvalidMeasurement);  // Bust has been replaced by Chest after the 5.6 preview 1 and before the 5.6 release
				}
				const float Underbust = Measurements.FindRef(TEXT("Underbust"), InvalidMeasurement);
				const float Waist = Measurements.FindRef(TEXT("Waist"), InvalidMeasurement);
				const float NeckToWaist = Measurements.FindRef(TEXT("Neck to Waist"), InvalidMeasurement);
				const float Rise = Measurements.FindRef(TEXT("Rise"), InvalidMeasurement);
				const float Inseam = Measurements.FindRef(TEXT("Inseam"), InvalidMeasurement);

				if (Hip != InvalidMeasurement &&
					Bust != InvalidMeasurement &&
					Underbust != InvalidMeasurement &&
					Waist != InvalidMeasurement &&
					NeckToWaist != InvalidMeasurement &&
					Rise != InvalidMeasurement &&
					Inseam != InvalidMeasurement &&
					true)
				{
					// Protruding
					constexpr float MaxProtrudingRatio = 0.88f;
					bIsProtruding = (Underbust / Bust) <= MaxProtrudingRatio;
					UE_LOGF(LogChaosOutfitAsset, Verbose,
						"BodyMatchParameters: Bust = %.3f, Underbust = %.3f, ProtrudingRatio = %.3f, bIsProtruding = %d",
						Bust, Underbust, Underbust / Bust, (int)bIsProtruding);

					// Torso shape
					const float HipBust = Hip / Bust;
					const float WaistRatio = ((Bust + Hip) / 2.f) / Waist;

					if (HipBust > 1.1f)
					{
						BodyShape = EBodyShape::Triangle;
					}
					else if (HipBust < 0.9f)
					{
						BodyShape = EBodyShape::InvertedTriangle;
					}
					else  // if (HipBust >= 0.9f && HipBust <= 1.1f)
					{
						if (WaistRatio > 1.3f)
						{
							BodyShape = EBodyShape::Circle;
						}
						else if (WaistRatio < 0.92f)
						{
							BodyShape = EBodyShape::Hourglass;
						}
						else  // if (WaistRatio >= 0.92f && WaistRatio <= 1.3f)
						{
							BodyShape = EBodyShape::Rectangle;
						}
					}

					// Width
					Width = FMath::Max(Hip, Waist, Bust) / 2.f;

					// Proportions
					ProportionTop = (NeckToWaist + (Rise / 2.f)) / Width;
					ProportionBottom = Inseam / (Hip / 2.f);
					ProportionFull = (NeckToWaist + (Rise / 2.f) + Inseam) / Width;
				}
			}

			static int32 Score(const FBodyMatchParameters& Garment, const FBodyMatchParameters& Body, const EBodyPlacement BodyPlacement = EBodyPlacement::Full)
			{
				if (Garment.BodyShape == EBodyShape::Invalid || Body.BodyShape == EBodyShape::Invalid)
				{
					return 0;
				}
				int32 Score = 0;

				// Protruding
				if (Garment.bIsProtruding == Body.bIsProtruding)
				{
					Score += 1000;
				}

				// Body shape
				switch (Garment.BodyShape)
				{
				case EBodyShape::Invalid: break;
				case EBodyShape::Rectangle:  // Hourglass > Inverted Tri > Tri > Circle
					switch (Body.BodyShape)
					{
					case EBodyShape::Invalid: break;
					case EBodyShape::Rectangle: Score += 500; break;
					case EBodyShape::Hourglass: Score += 400; break;
					case EBodyShape::InvertedTriangle: Score += 300; break;
					case EBodyShape::Triangle: Score += 200; break;
					case EBodyShape::Circle: Score += 100; break;
					}
					break;
				case EBodyShape::Circle:  // Rectangle > Triangle > InvertedTriangle > Hourglass
					switch (Body.BodyShape)
					{
					case EBodyShape::Invalid: break;
					case EBodyShape::Circle: Score += 500; break;
					case EBodyShape::Rectangle: Score += 400; break;
					case EBodyShape::Triangle: Score += 300; break;
					case EBodyShape::InvertedTriangle: Score += 200; break;
					case EBodyShape::Hourglass: Score += 100; break;
					}
					break;
				case EBodyShape::Hourglass:  // Triangle > Rectangle > InvertedTriangle > Circle
					switch (Body.BodyShape)
					{
					case EBodyShape::Invalid: break;
					case EBodyShape::Hourglass: Score += 500; break;
					case EBodyShape::Triangle: Score += 400; break;
					case EBodyShape::Rectangle: Score += 300; break;
					case EBodyShape::InvertedTriangle: Score += 200; break;
					case EBodyShape::Circle: Score += 100; break;
					}
					break;
				case EBodyShape::Triangle:  // Hourglass > Circle > Rectangle > InvertedTriangle
					switch (Body.BodyShape)
					{
					case EBodyShape::Invalid: break;
					case EBodyShape::Triangle: Score += 500; break;
					case EBodyShape::Hourglass: Score += 400; break;
					case EBodyShape::Circle: Score += 300; break;
					case EBodyShape::Rectangle: Score += 200; break;
					case EBodyShape::InvertedTriangle: Score += 100; break;
					}
					break;
				case EBodyShape::InvertedTriangle:  // Rectangle > Circle > Hourglass > Triangle
					switch (Body.BodyShape)
					{
					case EBodyShape::Invalid: break;
					case EBodyShape::InvertedTriangle: Score += 500; break;
					case EBodyShape::Rectangle: Score += 400; break;
					case EBodyShape::Circle: Score += 300; break;
					case EBodyShape::Hourglass: Score += 200; break;
					case EBodyShape::Triangle: Score += 100; break;
					}
					break;
				}

				// Proportion
				float ProportionDelta;
				switch (BodyPlacement)
				{
				default:
				case EBodyPlacement::Full: ProportionDelta = FMath::Abs(Garment.ProportionFull - Body.ProportionFull); break;
				case EBodyPlacement::Top: ProportionDelta = FMath::Abs(Garment.ProportionTop - Body.ProportionTop); break;
				case EBodyPlacement::Bottom: ProportionDelta = FMath::Abs(Garment.ProportionBottom - Body.ProportionBottom); break;
				}
				if (ProportionDelta <= 0.1f)
				{
					Score += 50;
				}
				else if (ProportionDelta <= 0.5f)
				{
					Score += (int32)(5.f / ProportionDelta);  // 50 to 10
				}
				// else Score += 0

				// Hip comparison
				Score -= (int32)(5.f * FMath::Abs(Garment.Hip - Body.Hip));  // 5 points per cm penalty

				return Score;
			}

		private:
			enum class EBodyShape : uint8 { Invalid, Rectangle, Circle, Hourglass, Triangle, InvertedTriangle };
			static inline const float InvalidMeasurement = UChaosOutfitAssetBodyUserData::InvalidMeasurement;

			float Hip = 0.f;
			float Width = 0.f;
			float ProportionTop = 0.f;
			float ProportionBottom = 0.f;
			float ProportionFull = 0.f;
			EBodyShape BodyShape = EBodyShape::Invalid;
			bool bIsProtruding = false;
		};
	}  // namespace Private

	FCollectionOutfitConstFacade::FCollectionOutfitConstFacade(const TSharedRef<const FManagedArrayCollection>& InManagedArrayCollection)
		: OutfitCollection(MakeUnique<const Private::FOutfitCollection>(const_cast<FManagedArrayCollection&>(InManagedArrayCollection.Get())))
		, ManagedArrayCollection(InManagedArrayCollection.ToSharedPtr())
	{}

	FCollectionOutfitConstFacade::FCollectionOutfitConstFacade(const FManagedArrayCollection& InManagedArrayCollection)
		: OutfitCollection(MakeUnique<const Private::FOutfitCollection>(const_cast<FManagedArrayCollection&>(InManagedArrayCollection)))
	{}

	FCollectionOutfitConstFacade::~FCollectionOutfitConstFacade() = default;

	bool FCollectionOutfitConstFacade::IsValid() const
	{
		return GetOutfitCollection()->IsValid();
	}

	TArray<FGuid> FCollectionOutfitConstFacade::GetOutfitGuids() const
	{
		check(IsValid());
		TArray<FGuid> OutfitGuids;
		OutfitGuids.Reserve(GetOutfitCollection()->Outfits.Guid->Num());
		for (const FGuid& Guid : *GetOutfitCollection()->Outfits.Guid)
		{
			OutfitGuids.AddUnique(Guid);
		}
		return OutfitGuids;
	}

	bool FCollectionOutfitConstFacade::HasValidBodySize(bool bBodyPartMustExist, bool bBodyMeasurementsMustExist, bool bInterpolationDataMustExist) const
	{
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

		for (int32 BodySize = 0; BodySize < GetNumBodySizes(); ++BodySize)
		{
			const TConstArrayView<FSoftObjectPath> BodyPartsSkeletalMeshes = GetBodySizeBodyPartsSkeletalMeshPaths(BodySize);
			if (!Algo::AnyOf(BodyPartsSkeletalMeshes, [&AssetRegistryModule, bBodyPartMustExist](const FSoftObjectPath& BodyPartsSkeletalMesh)
				{
					if (bBodyPartMustExist)
					{
						const FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(BodyPartsSkeletalMesh);
						return AssetData.IsValid();  //  Body part exists
					}
					return BodyPartsSkeletalMesh.IsValid();  // Body part is not empty
				}))
			{
				continue;  // Invalid body parts, try another size
			}

			if (bBodyMeasurementsMustExist)
			{
				const TMap<FString, float> BodySizeMeasurements = GetBodySizeMeasurements(BodySize);
				if (BodySizeMeasurements.IsEmpty() || Algo::AllOf(BodySizeMeasurements, [](const TPair<FString, float>& Measurement)
					{
						return Measurement.Value == UChaosOutfitAssetBodyUserData::InvalidMeasurement;
					}))
				{
					continue;  // No measurement, or all measurement values are 0, try another size
				}
			}

			if (bInterpolationDataMustExist)
			{
				const FRBFInterpolationDataWrapper RBFInterpolationDataWrapper = GetBodySizeInterpolationData(BodySize);
				if (!RBFInterpolationDataWrapper.SampleIndices.Num() ||
					!RBFInterpolationDataWrapper.SampleRestPositions.Num() ||
					!RBFInterpolationDataWrapper.InterpolationWeights.Num())
				{
					continue;  // Needs RBF data, try another size
				}
			}

			return true;  // Found at least one good size
		}
		return false;
	}

	TArray<FSoftObjectPath> FCollectionOutfitConstFacade::GetOutfitBodyPartsSkeletalMeshPaths() const
	{
		check(IsValid());
		TArray<FSoftObjectPath> BodyParts;
		BodyParts.Reserve(GetOutfitCollection()->BodyParts.SkeletalMeshPath->Num());
		for (const FSoftObjectPath& SkeletalMesh : (*GetOutfitCollection()->BodyParts.SkeletalMeshPath))
		{
			BodyParts.AddUnique(SkeletalMesh);
		}
		return BodyParts;
	}

	TArray<FString> FCollectionOutfitConstFacade::GetOutfitBodyPartsSkeletalMeshes() const
	{
		const TArray<FSoftObjectPath> BodyPartPaths = GetOutfitBodyPartsSkeletalMeshPaths();
		TArray<FString> BodyParts;
		BodyParts.Reserve(BodyPartPaths.Num());
		for (const FSoftObjectPath& SkeletalMesh : BodyPartPaths)
		{
			BodyParts.Emplace(SkeletalMesh.ToString());
		}
		return BodyParts;
	}

	TArray<int32> FCollectionOutfitConstFacade::GetOutfitBodySizes(const FGuid& Guid) const
	{
		check(IsValid());
		const int32 NumOutfits = GetOutfitCollection()->Outfits.Guid->Num();
		TArray<int32> OutfitBodySizes;
		OutfitBodySizes.Reserve(NumOutfits);
		for (int32 OutfitIndex = 0; OutfitIndex < NumOutfits; ++OutfitIndex)
		{
			if ((*GetOutfitCollection()->Outfits.Guid)[OutfitIndex] == Guid)
			{
				OutfitBodySizes.Add((*GetOutfitCollection()->Outfits.BodySize)[OutfitIndex]);
			}
		}
		return OutfitBodySizes;
	}

	const FString& FCollectionOutfitConstFacade::GetOutfitName(const FGuid& Guid, const int32 BodySize) const
	{
		for (int32 OutfitIndex = 0; OutfitIndex < GetOutfitCollection()->Outfits.Guid->Num(); ++OutfitIndex)
		{
			if ((*GetOutfitCollection()->Outfits.Guid)[OutfitIndex] == Guid &&
				(*GetOutfitCollection()->Outfits.BodySize)[OutfitIndex] == BodySize)
			{
				return (*GetOutfitCollection()->Outfits.Name)[OutfitIndex];
			}
		}
		static const FString EmptyString;
		return EmptyString;
	}

	TConstArrayView<FGuid> FCollectionOutfitConstFacade::GetOutfitPiecesGuids() const
	{
		return TConstArrayView<FGuid>(GetOutfitCollection()->Pieces.Guid->GetData(), GetOutfitCollection()->Pieces.Guid->Num());
	}

	TMap<FGuid, FString> FCollectionOutfitConstFacade::GetOutfitPieces(const FGuid& Guid, const int32 BodySize) const
	{
		check(IsValid());

		auto GetOutfitPiecesRange = [this](const FGuid& Guid, const int32 BodySize) -> FIntVector2
			{
				for (int32 OutfitIndex = 0; OutfitIndex < GetOutfitCollection()->Outfits.Guid->Num(); ++OutfitIndex)
				{
					if ((*GetOutfitCollection()->Outfits.Guid)[OutfitIndex] == Guid &&
						(*GetOutfitCollection()->Outfits.BodySize)[OutfitIndex] == BodySize)
					{
						return FIntVector2(
							(*GetOutfitCollection()->Outfits.PiecesStart)[OutfitIndex],
							(*GetOutfitCollection()->Outfits.PiecesCount)[OutfitIndex]);
					}
				}
				return FIntVector2(INDEX_NONE, 0);
			};

		const FIntVector2 Range = GetOutfitPiecesRange(Guid, BodySize);
		const int32 PiecesStart = Range[0];
		const int32 PiecesCount = Range[1];
		const TConstArrayView<FGuid> Guids =
			PiecesCount &&
			GetOutfitCollection()->Pieces.Guid->IsValidIndex(PiecesStart) &&
			GetOutfitCollection()->Pieces.Guid->IsValidIndex(PiecesStart + PiecesCount - 1) ?
			TConstArrayView<FGuid>(GetOutfitCollection()->Pieces.Guid->GetData() + PiecesStart, PiecesCount) :
			TConstArrayView<FGuid>();
		const TConstArrayView<FString> Names =
			PiecesCount &&
			GetOutfitCollection()->Pieces.Name->IsValidIndex(PiecesStart) &&
			GetOutfitCollection()->Pieces.Name->IsValidIndex(PiecesStart + PiecesCount - 1) ?
			TConstArrayView<FString>(GetOutfitCollection()->Pieces.Name->GetData() + PiecesStart, PiecesCount) :
			TConstArrayView<FString>();
		check(Guids.Num() == Names.Num());

		TMap<FGuid, FString> OutfitPieces;
		OutfitPieces.Reserve(Guids.Num());
		for (int32 Index = 0; Index < Guids.Num(); ++Index)
		{
			OutfitPieces.Emplace(Guids[Index], Names[Index]);
		}
		return OutfitPieces;
	}

	int32 FCollectionOutfitConstFacade::GetNumBodySizes() const
	{
		check(IsValid());
		return GetOutfitCollection()->BodySizes.Name->Num();
	}

	int32 FCollectionOutfitConstFacade::FindBodySize(const FString& Name) const
	{
		return GetOutfitCollection()->BodySizes.Name->Find(Name);
	}

	int32 FCollectionOutfitConstFacade::FindClosestBodySize(const TMap<FString, float>& Measurements) const
	{
		check(IsValid());
		const int32 NumBodySizes = GetNumBodySizes();

		switch (NumBodySizes)
		{
		case 0: return INDEX_NONE;
		case 1: return 0;
		default: break;
		}

		const Private::FBodyMatchParameters BodyMatchParameters(Measurements);

		int32 ClosestBodySize = INDEX_NONE;
		int32 BestScore = TNumericLimits<int32>::Min();

		for (int32 BodySize = 0; BodySize < NumBodySizes; ++BodySize)
		{
			const Private::FBodyMatchParameters GarmentMatchParameters(GetBodySizeMeasurements(BodySize));

			const int32 Score = Private::FBodyMatchParameters::Score(GarmentMatchParameters, BodyMatchParameters);
			if (Score > BestScore)
			{
				ClosestBodySize = BodySize;
				BestScore = Score;
			}
		}

		return ClosestBodySize;
	}

	int32 FCollectionOutfitConstFacade::FindClosestBodySize(const USkeletalMesh& BodyPart) const
	{
		if (BodyPart.GetAssetUserDataArray())
		{
			for (const UAssetUserData* const AssetUserData : *BodyPart.GetAssetUserDataArray())
			{
				if (const UChaosOutfitAssetBodyUserData* const BodyAssetUserData =
					Cast<const UChaosOutfitAssetBodyUserData>(AssetUserData))
				{
					return FindClosestBodySize(BodyAssetUserData->Measurements);
				}
			}
		}
		return INDEX_NONE;
	}

	const FString& FCollectionOutfitConstFacade::GetBodySizeName(int32 BodySize) const
	{
		check(IsValid());
		static const FString EmptyString;
		return GetOutfitCollection()->BodySizes.Name->IsValidIndex(BodySize) ?
			(*GetOutfitCollection()->BodySizes.Name)[BodySize] : EmptyString;
	}

	TConstArrayView<FSoftObjectPath> FCollectionOutfitConstFacade::GetBodySizeBodyPartsSkeletalMeshPaths(int32 BodySize) const
	{
		check(IsValid());
		if (GetOutfitCollection()->BodySizes.Name->IsValidIndex(BodySize))
		{
			const int32 BodyPartsStart = (*GetOutfitCollection()->BodySizes.BodyPartsStart)[BodySize];
			const int32 BodyPartsCount = (*GetOutfitCollection()->BodySizes.BodyPartsCount)[BodySize];
			if (BodyPartsCount &&
				GetOutfitCollection()->BodyParts.SkeletalMeshPath->IsValidIndex(BodyPartsStart) &&
				GetOutfitCollection()->BodyParts.SkeletalMeshPath->IsValidIndex(BodyPartsStart + BodyPartsCount - 1))
			{
				return TConstArrayView<FSoftObjectPath>(GetOutfitCollection()->BodyParts.SkeletalMeshPath->GetData() + BodyPartsStart, BodyPartsCount);
			}
		}
		return TConstArrayView<FSoftObjectPath>();
	}
	
	int32 FCollectionOutfitConstFacade::GetBodySizeBodyPartOffset(const int32 BodySize) const
	{
		check(IsValid());
		if (GetOutfitCollection()->BodySizes.BodyPartsStart->IsValidIndex(BodySize))
		{
			return (*GetOutfitCollection()->BodySizes.BodyPartsStart)[BodySize];
		}
		return INDEX_NONE;
	}

	int32 FCollectionOutfitConstFacade::GetBodySizeBodyPartCount(const int32 BodySize) const
	{
		check(IsValid());
		if (GetOutfitCollection()->BodySizes.BodyPartsStart->IsValidIndex(BodySize))
		{
			return (*GetOutfitCollection()->BodySizes.BodyPartsCount)[BodySize];
		}
		return 0;
	}

	FCollectionOutfitConstFacade::FRBFInterpolationDataWrapper FCollectionOutfitConstFacade::GetBodySizeInterpolationData(const int32 BodySize) const
	{
		check(IsValid());
		FRBFInterpolationDataWrapper Result;
		if (GetOutfitCollection()->BodySizes.BodyPartsStart->IsValidIndex(BodySize))
		{
			const int32 BodyPartsStart = (*GetOutfitCollection()->BodySizes.BodyPartsStart)[BodySize];
			const int32 BodyPartsCount = (*GetOutfitCollection()->BodySizes.BodyPartsCount)[BodySize];
			if (BodyPartsCount &&
				GetOutfitCollection()->BodyParts.RBFInterpolationSampleIndices->IsValidIndex(BodyPartsStart) &&
				GetOutfitCollection()->BodyParts.RBFInterpolationSampleIndices->IsValidIndex(BodyPartsStart + BodyPartsCount - 1))
			{
				Result.SampleIndices = TConstArrayView<TArray<int32>>(GetOutfitCollection()->BodyParts.RBFInterpolationSampleIndices->GetData() + BodyPartsStart, BodyPartsCount);
				Result.SampleRestPositions = TConstArrayView<TArray<FVector3f>>(GetOutfitCollection()->BodyParts.RBFInterpolationSampleRestPositions->GetData() + BodyPartsStart, BodyPartsCount);
				Result.InterpolationWeights = TConstArrayView<TArray<float>>(GetOutfitCollection()->BodyParts.RBFInterpolationWeights->GetData() + BodyPartsStart, BodyPartsCount);
			}
		}
		return Result;
	}

	TMap<FString, float> FCollectionOutfitConstFacade::GetBodySizeMeasurements(const int32 BodySize) const
	{
		using namespace OutfitCollection;
		check(IsValid());
		TMap<FString, float> Measurements;
		if (GetOutfitCollection()->BodySizes.Name->IsValidIndex(BodySize))
		{
			const FName Name((*GetOutfitCollection()->BodySizes.Name)[BodySize]);
			if (const TManagedArray<float>* Values = GetOutfitCollection()->ManagedArrayCollection.FindAttributeTyped<float>(Name, Group::Measurements))
			{
				const int32 NumMeasurements = GetOutfitCollection()->Measurements.Name->Num();
				Measurements.Reserve(NumMeasurements);
				for (int32 Index = 0; Index < NumMeasurements; ++Index)
				{
					Measurements.Emplace((*GetOutfitCollection()->Measurements.Name)[Index], (*Values)[Index]);
				}
			}
		}
		return Measurements;
	}

	FCollectionOutfitFacade::FCollectionOutfitFacade(const TSharedRef<FManagedArrayCollection>& ManagedArrayCollection) :
		FCollectionOutfitConstFacade(ManagedArrayCollection)
	{
	}

	FCollectionOutfitFacade::FCollectionOutfitFacade(FManagedArrayCollection& InManagedArrayCollection)
		: FCollectionOutfitConstFacade(InManagedArrayCollection)
	{
	}

	void FCollectionOutfitFacade::PostSerialize(const FArchive& Ar)
	{
		GetOutfitCollection()->PostSerialize(Ar);
	}

	FCollectionOutfitFacade::~FCollectionOutfitFacade() = default;

	void FCollectionOutfitFacade::DefineSchema()
	{
		GetOutfitCollection()->DefineSchema();
		check(IsValid());
	}

	void FCollectionOutfitFacade::Reset()
	{
		GetOutfitCollection()->Reset();
		check(IsValid());
	}

	int32 FCollectionOutfitFacade::AddBodySize(const FString& Name, const TConstArrayView<FSoftObjectPath> BodyPartsSkeletalMeshes, const TMap<FString, float>& Measurements, const FRBFInterpolationDataWrapper& InterpolationData)
	{
		using namespace OutfitCollection;
		check(IsValid());

		int32 BodySize = GetOutfitCollection()->BodySizes.Name->Find(Name);
		if (BodySize == INDEX_NONE)
		{
			BodySize = GetOutfitCollection()->ManagedArrayCollection.AddElements(1, OutfitCollection::Group::BodySizes);
			(*GetOutfitCollection()->BodySizes.Name)[BodySize] = Name;
		}
		// Add or replace body parts
		int32& BodyPartsCount = (*GetOutfitCollection()->BodySizes.BodyPartsCount)[BodySize];
		int32& BodyPartsStart = (*GetOutfitCollection()->BodySizes.BodyPartsStart)[BodySize];
		if (BodyPartsCount && BodyPartsStart != INDEX_NONE)
		{
			GetOutfitCollection()->ManagedArrayCollection.RemoveElements(OutfitCollection::Group::BodyParts, BodyPartsCount, BodyPartsStart);
		}

		BodyPartsCount = BodyPartsSkeletalMeshes.Num();
		if (BodyPartsCount)
		{
			BodyPartsStart = GetOutfitCollection()->ManagedArrayCollection.AddElements(BodyPartsCount, Group::BodyParts);
			const bool bInterpolationDataValid =
				InterpolationData.SampleIndices.Num() == BodyPartsCount &&
				InterpolationData.SampleRestPositions.Num() == BodyPartsCount &&
				InterpolationData.InterpolationWeights.Num() == BodyPartsCount;
				
			for (int32 Index = 0; Index < BodyPartsCount; ++Index)
			{
				(*GetOutfitCollection()->BodyParts.SkeletalMeshPath)[BodyPartsStart + Index] = BodyPartsSkeletalMeshes[Index];
				if (bInterpolationDataValid)
				{
					(*GetOutfitCollection()->BodyParts.RBFInterpolationSampleIndices)[BodyPartsStart + Index] = InterpolationData.SampleIndices[Index];
					(*GetOutfitCollection()->BodyParts.RBFInterpolationSampleRestPositions)[BodyPartsStart + Index] = InterpolationData.SampleRestPositions[Index];
					(*GetOutfitCollection()->BodyParts.RBFInterpolationWeights)[BodyPartsStart + Index] = InterpolationData.InterpolationWeights[Index];
				}
			}
		}
		else
		{
			check(BodyPartsStart == INDEX_NONE);
		}

		// Add or replace body size measurements
		const FName AttributeName(Name);
		TManagedArray<float>* Values = GetOutfitCollection()->ManagedArrayCollection.FindAttributeTyped<float>(AttributeName, Group::Measurements);
		if (!Values)
		{
			Values = &GetOutfitCollection()->ManagedArrayCollection.AddAttribute<float>(AttributeName, Group::Measurements);
		}
		else
		{
			// Clear all existing measurements
			Values->Fill(UChaosOutfitAssetBodyUserData::InvalidMeasurement);
		}
		for (const TPair<FString, float>& Measurement : Measurements)
		{
			int32 Index = GetOutfitCollection()->Measurements.Name->Find(Measurement.Key);
			if (Index == INDEX_NONE)
			{
				// Add missing key
				Index = GetOutfitCollection()->ManagedArrayCollection.AddElements(1, OutfitCollection::Group::Measurements);
				(*GetOutfitCollection()->Measurements.Name)[Index] = Measurement.Key;
			}
			(*Values)[Index] = Measurement.Value;
		}

		return BodySize;
	}

	int32 FCollectionOutfitFacade::AddBodySize(const FString& Name, const TConstArrayView<FString> BodyPartsSkeletalMeshes, const TMap<FString, float>& Measurements, const FRBFInterpolationDataWrapper& InterpolationData)
	{
		TArray<FSoftObjectPath> SKMPaths;
		SKMPaths.Reserve(BodyPartsSkeletalMeshes.Num());
		for (const FString& SKM : BodyPartsSkeletalMeshes)
		{
			SKMPaths.Emplace(SKM);
		}

		return AddBodySize(Name, SKMPaths, Measurements, InterpolationData);
	}

	void FCollectionOutfitFacade::AddOutfit(const FGuid& Guid, const int32 BodySize, const FString& Name, const TMap<FGuid, FString>& Pieces)
	{
		using namespace OutfitCollection;
		check(IsValid());

		auto FindOutfitIndex = [this](const FGuid& Guid, const int32 BodySize) -> int32
			{
				for (int32 Index = 0; Index < GetOutfitCollection()->Outfits.Guid->Num(); ++Index)
				{
					if ((*GetOutfitCollection()->Outfits.Guid)[Index] == Guid &&
						(*GetOutfitCollection()->Outfits.BodySize)[Index] == BodySize)
					{
						return Index;
					}
				}
				return INDEX_NONE;
			};

		int32 OutfitIndex = FindOutfitIndex(Guid, BodySize);
		if (OutfitIndex == INDEX_NONE)
		{
			OutfitIndex = GetOutfitCollection()->ManagedArrayCollection.AddElements(1, OutfitCollection::Group::Outfits);
			(*GetOutfitCollection()->Outfits.Guid)[OutfitIndex] = Guid;
			(*GetOutfitCollection()->Outfits.BodySize)[OutfitIndex] = BodySize;
		}
		// Add or replace name
		(*GetOutfitCollection()->Outfits.Name)[OutfitIndex] = Name;

		// Add or replace pieces
		int32& PiecesCount = (*GetOutfitCollection()->Outfits.PiecesCount)[OutfitIndex];
		int32& PiecesStart = (*GetOutfitCollection()->Outfits.PiecesStart)[OutfitIndex];
		if (PiecesStart != INDEX_NONE && PiecesCount)
		{
			GetOutfitCollection()->ManagedArrayCollection.RemoveElements(OutfitCollection::Group::Pieces, PiecesCount, PiecesStart);
		}
		PiecesCount = Pieces.Num();
		if (PiecesCount)
		{
			PiecesStart = GetOutfitCollection()->ManagedArrayCollection.AddElements(PiecesCount, Group::Pieces);
			int32 Index = PiecesStart;
			for (const TPair<FGuid, FString>& Piece : Pieces)
			{
				(*GetOutfitCollection()->Pieces.Guid)[Index] = Piece.Key;
				(*GetOutfitCollection()->Pieces.Name)[Index] = Piece.Value;
				++Index;
			}
		}
		else
		{
			check(PiecesStart == INDEX_NONE);
		}
	}

	void FCollectionOutfitFacade::AddOutfit(const FGuid& Guid, const int32 BodySize, const UChaosClothAssetBase& ClothAssetBase)
	{
		TMap<FGuid, FString> PieceInfos;
		const int32 NumPieces = ClothAssetBase.GetNumClothSimulationModels();
		PieceInfos.Reserve(NumPieces);
		for (int32 Index = 0; Index < NumPieces; ++Index)
		{
			PieceInfos.Emplace(
				ClothAssetBase.GetAssetGuid(Index),
				ClothAssetBase.GetClothSimulationModelName(Index).ToString());
		}

		AddOutfit(
			Guid,
			BodySize,
			ClothAssetBase.GetName(),
			PieceInfos);
	}

	void FCollectionOutfitFacade::Append(const FCollectionOutfitConstFacade& Other, int32 SelectedBodySize)
	{
		check(IsValid());
		if (!Other.IsValid())
		{
			return;
		}

		// Merge body sizes
		int32 NumBodySizes;
		if (SelectedBodySize == INDEX_NONE)
		{
			// Merge all the sizes
			NumBodySizes = Other.GetNumBodySizes();
			SelectedBodySize = 0;
		}
		else
		{
			// Merge the selected size
			NumBodySizes = 1;
			check(SelectedBodySize >= 0 && SelectedBodySize < Other.GetNumBodySizes());
		}

		TArray<int32> RemappedBodySize;
		RemappedBodySize.SetNumUninitialized(NumBodySizes);
		for (int32 BodySize = SelectedBodySize; BodySize < SelectedBodySize + NumBodySizes; ++BodySize)
		{
			UE_CLOGF(HasBodySize(Other.GetBodySizeName(BodySize)), LogChaosOutfitAsset, Display,
				"Outfit Collection Append operation caused body size [%ls] to be replaced.", *Other.GetBodySizeName(BodySize));

			// Add or replace the named body size
			RemappedBodySize[BodySize - SelectedBodySize] = AddBodySize(
				Other.GetBodySizeName(BodySize),
				Other.GetBodySizeBodyPartsSkeletalMeshPaths(BodySize),
				Other.GetBodySizeMeasurements(BodySize),
				Other.GetBodySizeInterpolationData(BodySize));
		}

		// Merge sized outfits
		for (const FGuid& Guid : Other.GetOutfitGuids())
		{
			for (const int32 BodySize : Other.GetOutfitBodySizes(Guid))
			{
				if (NumBodySizes > 1 || BodySize == SelectedBodySize)
				{
					AddOutfit(
						Guid,
						RemappedBodySize[BodySize - SelectedBodySize],
						Other.GetOutfitName(Guid, BodySize),
						Other.GetOutfitPieces(Guid, BodySize));
				}
			}
		}
	}
}	// namespace UE::Chaos::OutfitAsset

#undef LOCTEXT_NAMESPACE
