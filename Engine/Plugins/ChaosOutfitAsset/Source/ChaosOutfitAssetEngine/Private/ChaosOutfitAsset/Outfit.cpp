// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosOutfitAsset/Outfit.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosClothAsset/ClothEngineTools.h"
#include "ChaosClothAsset/ClothSimulationModel.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosOutfitAsset/BodyUserData.h"
#include "ChaosOutfitAsset/CollectionOutfitFacade.h"
#include "ChaosOutfitAsset/OutfitAsset.h"
#include "ChaosOutfitAsset/OutfitAssetPrivate.h"
#include "ChaosOutfitAsset/OutfitAssetUtility.h"
#include "ChaosOutfitAsset/OutfitCollection.h"
#include "ChaosOutfitAsset/SizedOutfitSource.h"
#include "Engine/SkeletalMesh.h"
#include "MeshResizing/RBFInterpolation.h"
#include "Misc/ScopedSlowTask.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "StaticMeshResources.h"
#include "Tasks/Task.h"
#include "Async/TaskGraphInterfaces.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Outfit)

#define LOCTEXT_NAMESPACE "ChaosOutfit"

struct UChaosOutfit::FLODRenderData
{
	// Half edge buffers are always regenerated on the last merge, a dummy buffer is used until then
	struct FDummyHalfEdgeBuffer
	{
		bool bHasHalfEdges = false;
		
		bool IsCPUDataValid() const
		{
			return bHasHalfEdges;
		}
		void CleanUp()
		{
			bHasHalfEdges = false;
		}
		void Init(UChaosOutfit::FLODRenderData&)
		{
			bHasHalfEdges = true;
		}
	};

	TArray<FSkelMeshRenderSection> RenderSections;
	FMultiSizeIndexContainer MultiSizeIndexContainer;
	FStaticMeshVertexBuffers StaticVertexBuffers;
	FSkinWeightVertexBuffer SkinWeightVertexBuffer;
	FSkeletalMeshVertexClothBuffer ClothVertexBuffer;
	FSkinWeightProfilesData SkinWeightProfilesData;
	TArray<FBoneIndexType> ActiveBoneIndices;
	TArray<FBoneIndexType> RequiredBones;
	FDummyHalfEdgeBuffer HalfEdgeBuffer;

	uint32 GetNumVertices() const
	{
		return StaticVertexBuffers.PositionVertexBuffer.GetNumVertices();
	}

	bool HasClothData() const
	{
		for (int32 SectionIdx = 0; SectionIdx < RenderSections.Num(); SectionIdx++)
		{
			if (RenderSections[SectionIdx].HasClothingData())
			{
				return true;
			}
		}
		return false;	
	}

	friend FArchive& operator<<(FArchive& Ar, UChaosOutfit::FLODRenderData& LODRenderData)
	{
		static const FName MultiSizeIndexContainerName(TEXT("UChaosOutfit::FLODRenderData.MultiSizeIndexContainer"));
		static const FName StaticPositionVertexBufferName(TEXT("UChaosOutfit::FLODRenderData.StaticVertexBuffers.PositionVertexBuffer"));
		static const FName StaticMeshVertexBufferName(TEXT("UChaosOutfit::FLODRenderData.StaticVertexBuffers.StaticMeshVertexBuffer"));
		static const FName StaticColorVertexBufferName(TEXT("UChaosOutfit::FLODRenderData.StaticVertexBuffers.ColorVertexBuffer"));
		static const FName ClothVertexBufferName(TEXT("UChaosOutfit::FLODRenderData.ClothVertexBuffer"));
		static const FName SourceRayTracingGeometryName(TEXT("UChaosOutfit::FLODRenderData.SourceRayTracingGeometry"));

		constexpr bool bNeedsCPUAccess = true;

		Ar << LODRenderData.RenderSections;

		LODRenderData.MultiSizeIndexContainer.Serialize(Ar, bNeedsCPUAccess);
		LODRenderData.MultiSizeIndexContainer.GetIndexBuffer()->SetResourceName(MultiSizeIndexContainerName);

		LODRenderData.StaticVertexBuffers.PositionVertexBuffer.Serialize(Ar, bNeedsCPUAccess);
		LODRenderData.StaticVertexBuffers.PositionVertexBuffer.SetResourceName(StaticPositionVertexBufferName);
		LODRenderData.StaticVertexBuffers.StaticMeshVertexBuffer.Serialize(Ar, bNeedsCPUAccess);
		LODRenderData.StaticVertexBuffers.StaticMeshVertexBuffer.SetResourceName(StaticMeshVertexBufferName);

		Ar << LODRenderData.SkinWeightVertexBuffer;
		if (LODRenderData.HasClothData()) // This requires that RenderSections have already been serialized/deserialized
		{
			Ar << LODRenderData.ClothVertexBuffer;
			LODRenderData.ClothVertexBuffer.SetResourceName(ClothVertexBufferName);
		}

		Ar << LODRenderData.ActiveBoneIndices;
		Ar << LODRenderData.RequiredBones;
		Ar << LODRenderData.HalfEdgeBuffer.bHasHalfEdges;

		return Ar;
	}
};

struct UChaosOutfit::FRenderData
{
	TIndirectArray<FLODRenderData> LODRenderData;
	uint8 NumInlinedLODs = 0;
	uint8 NumNonOptionalLODs = 0;
	bool bSupportRayTracing = false;

	friend FArchive& operator<<(FArchive& Ar, UChaosOutfit::FRenderData& InRenderData)
	{
		Ar << InRenderData.LODRenderData;
		Ar << InRenderData.NumInlinedLODs;
		Ar << InRenderData.NumNonOptionalLODs;
		Ar << InRenderData.bSupportRayTracing;

		return Ar;
	}
};

namespace UE::Chaos::OutfitAsset::Private
{
	template<typename T UE_REQUIRES(std::is_same_v<T, UChaosClothAssetBase> || std::is_same_v<T, TArray<FChaosOutfitPiece>>)>
	FChaosOutfitPiece Clone(const T& Pieces, int32 ModelIndex)
	{
		return FChaosOutfitPiece(Pieces, ModelIndex);
	}

	template<typename T UE_REQUIRES(std::is_same_v<T, UChaosClothAssetBase> || std::is_same_v<T, TArray<FChaosOutfitPiece>>)>
	FGuid GetAssetGuid(const T& Pieces, int32 ModelIndex);

	template<>
	FGuid GetAssetGuid<UChaosClothAssetBase>(const UChaosClothAssetBase& Pieces, int32 ModelIndex)
	{
		return Pieces.GetAssetGuid(ModelIndex);
	}

	template<>
	FGuid GetAssetGuid<TArray<FChaosOutfitPiece>>(const TArray<FChaosOutfitPiece>& Pieces, int32 ModelIndex)
	{
		return Pieces[ModelIndex].AssetGuid;
	}

	static void SerializeCollections(FArchive& Ar, TArray<TSharedRef<const FManagedArrayCollection>>& Collections, const FString& PieceName)
	{
		int32 NumCollections = Collections.Num();
		Ar << NumCollections;

		if (Ar.IsLoading())
		{
			Collections.Reset(NumCollections);
			for (int32 Index = 0; Index < NumCollections; ++Index)
			{
				TSharedRef<FManagedArrayCollection> NewCollection = MakeShared<FManagedArrayCollection>();
				NewCollection->Serialize(Ar);

				::Chaos::Softs::FCollectionPropertyMutableFacade MutablePropertyFacade(NewCollection);
				MutablePropertyFacade.PostSerialize(Ar);

				UE::Chaos::ClothAsset::FCollectionClothFacade ClothFacade(NewCollection);
				ClothFacade.PostSerialize(Ar);

				Collections.Emplace(MoveTemp(NewCollection));
			}
		}
		else
		{
#if WITH_EDITORONLY_DATA
			if (Ar.IsCooking() && Ar.IsSaving())
			{
				// Strip render-mesh fields from each piece's collection on cook, mirroring UChaosClothAsset's cook-time trim
				const bool bLog = !Ar.IsObjectReferenceCollector();  // Suppress logs during the package harvester pass, the cooker calls Serialize again on the linker save pass
				const FString OutfitPath = UE::Chaos::ClothAsset::FClothEngineTools::GetCookedAssetPath(Ar);
				for (int32 Index = 0; Index < NumCollections; ++Index)
				{
					const TSharedRef<const FManagedArrayCollection> Trimmed =
						UE::Chaos::ClothAsset::FClothEngineTools::TrimClothCollectionOnCook(
							FString::Printf(TEXT("%s OutfitPiece[%s]:[%d]"), *OutfitPath, *PieceName, Index),
							Collections[Index],
							bLog);
					ConstCastSharedRef<FManagedArrayCollection>(Trimmed)->Serialize(Ar);
				}
			}
			else
#endif
			{
				for (int32 Index = 0; Index < NumCollections; ++Index)
				{
					ConstCastSharedRef<FManagedArrayCollection>(Collections[Index])->Serialize(Ar);
				}
			}
		}
	}
}  // namespace UE::Chaos::OutfitAsset::Private

FChaosOutfitPiece::FChaosOutfitPiece()
	: ClothSimulationModel(MakeShared<FChaosClothSimulationModel>())
{
}

FChaosOutfitPiece::~FChaosOutfitPiece() = default;

FChaosOutfitPiece::FChaosOutfitPiece(
	FName InName,
	FGuid InAssetGuid,
	const UPhysicsAsset* InPhysicsAsset,
	const FChaosClothSimulationModel& InClothSimulationModel,
	const TArray<TSharedRef<const FManagedArrayCollection>>& InCollections)
	: Name(InName)
	, AssetGuid(InAssetGuid)
	, PhysicsAsset(InPhysicsAsset)
	, ClothSimulationModel(MakeShared<FChaosClothSimulationModel>(InClothSimulationModel))
{
	DeepCopyCollections(InCollections);
}

FChaosOutfitPiece::FChaosOutfitPiece(const UChaosClothAssetBase& ClothAssetBase, int32 ModelIndex)
	: FChaosOutfitPiece(
		ClothAssetBase.GetClothSimulationModelName(ModelIndex),
		ClothAssetBase.GetAssetGuid(ModelIndex),
		ClothAssetBase.GetPhysicsAssetForModel(ModelIndex),
		*ClothAssetBase.GetClothSimulationModel(ModelIndex),
		ClothAssetBase.GetCollections(ModelIndex))
{}

FChaosOutfitPiece::FChaosOutfitPiece(const FChaosOutfitPiece& Other)
	: FChaosOutfitPiece(
		Other.Name,
		Other.AssetGuid,
		Other.PhysicsAsset,
		*Other.ClothSimulationModel,
		Other.Collections)
{}

FChaosOutfitPiece::FChaosOutfitPiece(FChaosOutfitPiece&& Other)
	: Name(MoveTemp(Other.Name))
	, AssetGuid(MoveTemp(Other.AssetGuid))
	, PhysicsAsset(MoveTemp(Other.PhysicsAsset))
	, ClothSimulationModel(MoveTemp(Other.ClothSimulationModel))
	, Collections(MoveTemp(Other.Collections))
{}

FChaosOutfitPiece& FChaosOutfitPiece::operator=(const FChaosOutfitPiece& Other)
{
	if (&Other != this)
	{
		Name = Other.Name;
		PhysicsAsset = Other.PhysicsAsset;
		AssetGuid = Other.AssetGuid;
		*ClothSimulationModel = *Other.ClothSimulationModel;
		DeepCopyCollections(Other.Collections);
	}
	return *this;
}

FChaosOutfitPiece& FChaosOutfitPiece::operator=(FChaosOutfitPiece&& Other)
{
	if (&Other != this)
	{
		Name = MoveTemp(Other.Name);
		PhysicsAsset = MoveTemp(Other.PhysicsAsset);
		AssetGuid = MoveTemp(Other.AssetGuid);
		ClothSimulationModel = MoveTemp(Other.ClothSimulationModel);
		Collections = MoveTemp(Other.Collections);
	}
	return *this;
}

bool FChaosOutfitPiece::Serialize(FArchive& Ar)
{
	// Serialize what can be done with tagged properties
	{
		UScriptStruct* const Struct = FChaosOutfitPiece::StaticStruct();
		Struct->SerializeTaggedProperties(Ar, (uint8*)this, Struct, nullptr);
	}
	// Custom serialize the model since TSharedRef can't be declared as UPROPERTY
	{
		UScriptStruct* const Struct = FChaosClothSimulationModel::StaticStruct();
		Struct->SerializeTaggedProperties(Ar, (uint8*)&ClothSimulationModel.Get(), Struct, nullptr);
	}
	// Custom serialize the collections since TArray<TSharedRef> can't be declared as UPROPERTY
	UE::Chaos::OutfitAsset::Private::SerializeCollections(Ar, Collections, Name.ToString());
	return true;
}

void FChaosOutfitPiece::DeepCopyCollections(const TArray<TSharedRef<const FManagedArrayCollection>>& Other)
{
	Collections.Reserve(Other.Num());
	for (const TSharedRef<const FManagedArrayCollection>& Collection : Other)
	{
		Collections.Emplace(MakeShared<const FManagedArrayCollection>(*Collection));
	}
}

void FChaosOutfitPiece::RemapBoneIndices(const TArray<int32>& BoneMap)
{
	ClothSimulationModel->RemapBoneIndices(BoneMap);
}

UChaosOutfit::UChaosOutfit(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Init(
		Pieces,
		ReferenceSkeleton,
		RenderData,
		Materials,
		OutfitCollection);
}

UChaosOutfit::UChaosOutfit(FVTableHelper& Helper)
	: Super(Helper)
{}

UChaosOutfit::~UChaosOutfit() = default;

void UChaosOutfit::Append(const UChaosOutfit& Other, const FString& BodySizeNameFilter)
{
	Merge(
		Other.GetReferenceSkeleton(),
		Other.RenderData.Get(),
		Other.Materials,
		Other.OutfitCollection,
		Other.Pieces,
		Other.Pieces.Num(),
		BodySizeNameFilter,
		Pieces,
		ReferenceSkeleton,
		RenderData.Get(),
		Materials,
		OutfitCollection);
}

void UChaosOutfit::Add(const UChaosClothAssetBase& ClothAssetBase)
{
	using namespace UE::Chaos::OutfitAsset;

	// Retrieve the input outfit collection
	TOptional<FManagedArrayCollection> InOutfitCollection;
	const FManagedArrayCollection* InOutfitCollectionPtr = nullptr;

	if (const UChaosClothAsset* ClothAsset = ExactCast<const UChaosClothAsset>(&ClothAssetBase))
	{
		// Build some outfit metadata for this cloth asset
		FCollectionOutfitFacade InOutfitFacade(InOutfitCollection.Emplace());
		InOutfitFacade.DefineSchema();

		const FGuid OutfitGuid = FGuid::NewGuid();
		const int32 BodySize = InOutfitFacade.FindOrAddBodySize(DefaultBodySize.ToString());  // TODO: Should the default body size be part of the schema?
		InOutfitFacade.AddOutfit(OutfitGuid, BodySize, ClothAssetBase);

		InOutfitCollectionPtr = &InOutfitCollection.GetValue();
	}
	else if(const UChaosOutfitAsset* const OutfitAsset = ExactCast<const UChaosOutfitAsset>(&ClothAssetBase))
	{
		InOutfitCollectionPtr = &OutfitAsset->GetOutfitCollection();
	}
	check(InOutfitCollectionPtr);

	// Add cloth/outfit asset
	Merge(
		ClothAssetBase.GetRefSkeleton(),
		ClothAssetBase.GetResourceForRendering(),
		ClothAssetBase.GetMaterials(),
		*InOutfitCollectionPtr,
		ClothAssetBase,
		ClothAssetBase.GetNumClothSimulationModels(),
		{},
		Pieces,
		ReferenceSkeleton,
		RenderData.Get(),
		Materials,
		OutfitCollection);
}

void UChaosOutfit::Add(const FChaosSizedOutfitSource& SizedOutfitSource, const FGuid& OutfitGuid)
{
	using namespace UE::Chaos::OutfitAsset;

	// Add body size
	FManagedArrayCollection InOutfitCollection;
	FCollectionOutfitFacade InOutfitFacade(InOutfitCollection);
	InOutfitFacade.DefineSchema();

	int32 BodySize = INDEX_NONE;

	const FString BodySizeName = SizedOutfitSource.GetBodySizeName();
	if (!BodySizeName.IsEmpty())
	{
		TArray<FSoftObjectPath> BodyPartsSkeletalMeshes;
		TArray<TArray<int32>> InterpolationData_SampleIndices;
		TArray<TArray<FVector3f>> InterpolationData_SampleRestPositions;
		TArray<TArray<float>> InterpolationData_InterpolationWeights;
		BodyPartsSkeletalMeshes.Reserve(SizedOutfitSource.SourceBodyParts.Num());
		TMap<FString, float> Measurements;
		for (const TObjectPtr<const USkeletalMesh>& SourceBodyPart : SizedOutfitSource.SourceBodyParts)
		{
			if (SourceBodyPart)
			{
				if (SourceBodyPart->GetAssetUserDataArray())
				{
					for (const UAssetUserData* const AssetUserData : *SourceBodyPart->GetAssetUserDataArray())  // Not using the simpler GetAssetUserDataOfClass here because it is not const
					{
						if (const UChaosOutfitAssetBodyUserData* const BodyAssetUserData =
							Cast<const UChaosOutfitAssetBodyUserData>(AssetUserData))
						{
							Measurements.Append(BodyAssetUserData->Measurements);
							break;
						}
					}
				}
				BodyPartsSkeletalMeshes.Emplace(SourceBodyPart->GetPathName());
#if WITH_EDITORONLY_DATA
				if (SizedOutfitSource.NumResizingInterpolationPoints > 0)
				{
					FMeshResizingRBFInterpolationData InterpData;
					if (const FMeshDescription* const MeshDescription = SourceBodyPart->GetMeshDescription(0))
					{
						constexpr float NumSteps = 1.f;
						FScopedSlowTask SlowTask(NumSteps);
						SlowTask.MakeDialog();  // Can't delay the dialog or it won't show up
						SlowTask.EnterProgressFrame(NumSteps, LOCTEXT("GeneratingRBFInterpolationWeights", "Generating RBF interpolation weights (please wait, this can take several minutes)..."));
						SlowTask.TickProgress();
						SlowTask.ForceRefresh();
						UE::MeshResizing::FRBFInterpolation::GenerateWeights(*MeshDescription, SizedOutfitSource.NumResizingInterpolationPoints, InterpData);
					}
					InterpolationData_SampleIndices.Emplace(MoveTemp(InterpData.SampleIndices));
					InterpolationData_SampleRestPositions.Emplace(MoveTemp(InterpData.SampleRestPositions));
					InterpolationData_InterpolationWeights.Emplace(MoveTemp(InterpData.InterpolationWeights));
				}
#endif
			}
		}
		FCollectionOutfitConstFacade::FRBFInterpolationDataWrapper InterpolationData;
		InterpolationData.SampleIndices = TConstArrayView<TArray<int32>>(InterpolationData_SampleIndices);
		InterpolationData.SampleRestPositions = TConstArrayView<TArray<FVector3f>>(InterpolationData_SampleRestPositions);
		InterpolationData.InterpolationWeights = TConstArrayView<TArray<float>>(InterpolationData_InterpolationWeights);
		BodySize = InOutfitFacade.AddBodySize(BodySizeName, BodyPartsSkeletalMeshes, Measurements, InterpolationData);
	}
	else
	{
		BodySize = InOutfitFacade.FindOrAddBodySize(DefaultBodySize.ToString());
	}

	// Add source asset pieces
	if (const UChaosClothAssetBase* const ClothAssetBase = SizedOutfitSource.SourceAsset)
	{
		// Move the entire set of outfit pieces under the new size/GUID
		InOutfitFacade.AddOutfit(OutfitGuid, BodySize, *ClothAssetBase);

		// Add cloth/outfit asset
		Merge(
			ClothAssetBase->GetRefSkeleton(),
			ClothAssetBase->GetResourceForRendering(),
			ClothAssetBase->GetMaterials(),
			InOutfitCollection,
			*ClothAssetBase,
			ClothAssetBase->GetNumClothSimulationModels(),
			{},
			Pieces,
			ReferenceSkeleton,
			RenderData.Get(),
			Materials,
			OutfitCollection);
	}
	else
	{
		// Only add the size
		FCollectionOutfitFacade OutfitFacade(OutfitCollection);
		OutfitFacade.Append(InOutfitFacade);
	}
}

void UChaosOutfit::CopyTo(
	TArray<FChaosOutfitPiece>& OutPieces,
	FReferenceSkeleton& OutReferenceSkeleton,
	TUniquePtr<FSkeletalMeshRenderData>& OutSkeletalMeshRenderData,
	TArray<FSkeletalMaterial>& OutMaterials,
	FManagedArrayCollection& OutOutfitCollection) const
{
	Init(
		OutPieces,
		OutReferenceSkeleton,
		OutSkeletalMeshRenderData,
		OutMaterials,
		OutOutfitCollection);

	Merge(
		ReferenceSkeleton,
		RenderData.Get(),
		Materials,
		OutfitCollection,
		Pieces,
		Pieces.Num(),
		{},
		OutPieces,
		OutReferenceSkeleton,
		OutSkeletalMeshRenderData.Get(),
		OutMaterials,
		OutOutfitCollection);
}

int32 UChaosOutfit::GetNumLods() const
{
	int32 NumLods = 0;
	for (const FChaosOutfitPiece& Piece : Pieces)
	{
		NumLods = FMath::Max(Piece.Collections.Num(), NumLods);
	}
	return NumLods;
}

TArray<TSharedRef<const FManagedArrayCollection>> UChaosOutfit::GetClothCollections(int32 StartLodIndex) const
{
	int32 NumLods;
	if (StartLodIndex == INDEX_NONE)
	{
		// Get all LODs
		NumLods = GetNumLods();
		StartLodIndex = 0;
	}
	else
	{
		NumLods = 1;
	}

	TArray<TSharedRef<const FManagedArrayCollection>> ClothCollections;
	ClothCollections.Reserve(NumLods * Pieces.Num());

	TSharedRef<FManagedArrayCollection> EmptyClothCollection = MakeShared<FManagedArrayCollection>();
	UE::Chaos::ClothAsset::FCollectionClothFacade ClothFacade(EmptyClothCollection);
	ClothFacade.DefineSchema();

	for (const FChaosOutfitPiece& Piece : Pieces)
	{
		for (int32 LodIndex = StartLodIndex; LodIndex < StartLodIndex + NumLods; ++LodIndex)
		{
			ClothCollections.Emplace(Piece.Collections.IsValidIndex(LodIndex) ?
				Piece.Collections[LodIndex] :
				EmptyClothCollection);
		}
	}
	return ClothCollections;
}

bool UChaosOutfit::HasBodySize(const FString& SizeName) const
{
	UE::Chaos::OutfitAsset::FCollectionOutfitConstFacade OutfitFacade(OutfitCollection);
	return OutfitFacade.IsValid() && OutfitFacade.HasBodySize(SizeName);
}

void UChaosOutfit::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar << ReferenceSkeleton;
	Ar << *RenderData;
	if (Ar.IsLoading())
	{
		UE::Chaos::OutfitAsset::FCollectionOutfitFacade OutfitFacade(OutfitCollection);
		OutfitFacade.PostSerialize(Ar);
	}
}

template<typename RenderDataType UE_REQUIRES_DEFINITION(std::is_same_v<RenderDataType, FSkeletalMeshRenderData> || std::is_same_v<RenderDataType, UChaosOutfit::FRenderData>)>
void UChaosOutfit::Init(
	TArray<FChaosOutfitPiece>& OutPieces,
	FReferenceSkeleton& OutReferenceSkeleton,
	TUniquePtr<RenderDataType>& OutSkeletalMeshRenderData,
	TArray<FSkeletalMaterial>& OutMaterials,
	FManagedArrayCollection& OutOutfitCollection)
{
	OutPieces.Reset();
	OutReferenceSkeleton.Empty();
	OutSkeletalMeshRenderData = MakeUnique<RenderDataType>();
	OutMaterials.Reset();
	OutOutfitCollection.Reset();

	// Create a default valid reference skeleton
	OutReferenceSkeleton.Empty(1);
	FReferenceSkeletonModifier ReferenceSkeletonModifier(OutReferenceSkeleton, nullptr);
	FMeshBoneInfo MeshBoneInfo;
	constexpr const TCHAR* RootName = TEXT("Root");
	MeshBoneInfo.ParentIndex = INDEX_NONE;
#if WITH_EDITORONLY_DATA
	MeshBoneInfo.ExportName = RootName;
#endif
	MeshBoneInfo.Name = FName(RootName);
	ReferenceSkeletonModifier.Add(MeshBoneInfo, FTransform::Identity);

	// Set default values for the empty render data that can be usefully used in a merge
	OutSkeletalMeshRenderData->bSupportRayTracing = false;

	// Init outfit collection
	UE::Chaos::OutfitAsset::FCollectionOutfitFacade OutfitFacade(OutOutfitCollection);
	OutfitFacade.DefineSchema();
}

template<typename InLODRenderDataType, typename OutLODRenderDataType>
void UChaosOutfit::MergeLODRenderDatas(
	const InLODRenderDataType& LODRenderData,
	const TArray<FGuid>& AssetGuids,
	const int32 MaterialOffset,
	const FReferenceSkeleton& ReferenceSkeleton,
	const TArray<int32>& BoneMap,
	OutLODRenderDataType& OutLODRenderData)
{
	using namespace UE::Chaos::OutfitAsset;

	const uint32 VertexOffset = OutLODRenderData.GetNumVertices();
	const uint32 NumVertices = VertexOffset + LODRenderData.GetNumVertices();

	// Merge index buffer
	TArray<uint32> IndexBuffer = GetIndices(OutLODRenderData.MultiSizeIndexContainer);
	const uint32 IndexOffset = IndexBuffer.Num();

	IndexBuffer.Append(GetIndices(LODRenderData.MultiSizeIndexContainer, VertexOffset));
	const uint32 NumIndices = IndexBuffer.Num();

	const uint8 DataTypeSize = (NumVertices < (uint32)TNumericLimits<uint16>::Max()) ? sizeof(uint16) : sizeof(uint32);
	OutLODRenderData.MultiSizeIndexContainer.RebuildIndexBuffer(DataTypeSize, IndexBuffer);

	// Merge positions
	TArray<FVector3f> Positions = GetPositions(OutLODRenderData.StaticVertexBuffers.PositionVertexBuffer);
	Positions.Append(GetPositions(LODRenderData.StaticVertexBuffers.PositionVertexBuffer));
	check(Positions.Num() == NumVertices);

	// Merge tangents
	const bool bUseHighPrecisionTangentBasis =
		OutLODRenderData.StaticVertexBuffers.StaticMeshVertexBuffer.GetUseHighPrecisionTangentBasis() ||
		LODRenderData.StaticVertexBuffers.StaticMeshVertexBuffer.GetUseHighPrecisionTangentBasis();

	TArray<FVector4f> Tangents[3];
	for (int Axis = 0; Axis < 3; ++Axis)
	{
		Tangents[Axis] = GetTangents(OutLODRenderData.StaticVertexBuffers.StaticMeshVertexBuffer, Axis);
		Tangents[Axis].Append(GetTangents(LODRenderData.StaticVertexBuffers.StaticMeshVertexBuffer, Axis));
		check(Tangents[Axis].Num() == NumVertices);
	}

	// Merge UVs
	const uint32 MaxTexCoords = FMath::Max(
		LODRenderData.StaticVertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords(),
		OutLODRenderData.StaticVertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords());
	const bool bUseFullPrecisionUVs =
		OutLODRenderData.StaticVertexBuffers.StaticMeshVertexBuffer.GetUseFullPrecisionUVs() ||
		LODRenderData.StaticVertexBuffers.StaticMeshVertexBuffer.GetUseFullPrecisionUVs();

	TArray<FVector2f> VertexUVs = GetVertexUVs(OutLODRenderData.StaticVertexBuffers.StaticMeshVertexBuffer, MaxTexCoords);
	VertexUVs.Append(GetVertexUVs(LODRenderData.StaticVertexBuffers.StaticMeshVertexBuffer, MaxTexCoords));
	check(VertexUVs.Num() == NumVertices * MaxTexCoords);

	// Merge Vertex attributes

	// TODO: 
	//     The Cloth Asset doesn't have any vertex attributes,
	//     but if it had, merging them is non trivial due to no access to the names

	// Merge Vertex colors
	const bool bHasVertexColors = NumVertices && (
		OutLODRenderData.StaticVertexBuffers.ColorVertexBuffer.GetAllocatedSize() != 0 ||
		LODRenderData.StaticVertexBuffers.ColorVertexBuffer.GetAllocatedSize() != 0);

	TArray<FColor> VertexColors;
	if (bHasVertexColors)
	{
		VertexColors = GetVertexColors(OutLODRenderData.StaticVertexBuffers);
		VertexColors.Append(GetVertexColors(LODRenderData.StaticVertexBuffers));
		check(VertexColors.Num() == NumVertices);
	}

	// Init vertex buffers
	FPositionVertexBuffer& PositionVertexBuffer = OutLODRenderData.StaticVertexBuffers.PositionVertexBuffer;
	FStaticMeshVertexBuffer& StaticMeshVertexBuffer = OutLODRenderData.StaticVertexBuffers.StaticMeshVertexBuffer;

	StaticMeshVertexBuffer.SetUseFullPrecisionUVs(bUseFullPrecisionUVs);
	StaticMeshVertexBuffer.SetUseHighPrecisionTangentBasis(bUseHighPrecisionTangentBasis);

	PositionVertexBuffer.Init(NumVertices);
	StaticMeshVertexBuffer.Init(NumVertices, MaxTexCoords);

	for (uint32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
	{
		PositionVertexBuffer.VertexPosition(VertexIndex) = Positions[VertexIndex];
		StaticMeshVertexBuffer.SetVertexTangents(VertexIndex, Tangents[0][VertexIndex], Tangents[1][VertexIndex], Tangents[2][VertexIndex]);
		for (uint32 UVIndex = 0; UVIndex < MaxTexCoords; ++UVIndex)
		{
			StaticMeshVertexBuffer.SetVertexUV(VertexIndex, UVIndex, VertexUVs[VertexIndex * MaxTexCoords + UVIndex]);
		}
	}
	if (bHasVertexColors)
	{
		FColorVertexBuffer& ColorVertexBuffer = OutLODRenderData.StaticVertexBuffers.ColorVertexBuffer;
		ColorVertexBuffer.InitFromColorArray(VertexColors.GetData(), NumVertices);
	}

	// Skinweight buffer
	const bool bNeedsCPUAccess =
		OutLODRenderData.SkinWeightVertexBuffer.GetNeedsCPUAccess() ||
		LODRenderData.SkinWeightVertexBuffer.GetNeedsCPUAccess();
	const uint32 MaxBoneInfluences = FMath::Max(
		OutLODRenderData.SkinWeightVertexBuffer.GetMaxBoneInfluences(),
		LODRenderData.SkinWeightVertexBuffer.GetMaxBoneInfluences());
	const bool bUse16BitBoneIndex =
		OutLODRenderData.SkinWeightVertexBuffer.Use16BitBoneIndex() ||
		LODRenderData.SkinWeightVertexBuffer.Use16BitBoneIndex();
	const bool bUse16BitBoneWeight =
		OutLODRenderData.SkinWeightVertexBuffer.Use16BitBoneWeight() ||
		LODRenderData.SkinWeightVertexBuffer.Use16BitBoneWeight();

	TArray<FSkinWeightInfo> SkinWeights = GetSkinWeights(OutLODRenderData.SkinWeightVertexBuffer, bUse16BitBoneWeight);
	SkinWeights.Append(GetSkinWeights(LODRenderData.SkinWeightVertexBuffer, bUse16BitBoneWeight, &BoneMap));

	OutLODRenderData.SkinWeightVertexBuffer.SetNeedsCPUAccess(bNeedsCPUAccess);
	OutLODRenderData.SkinWeightVertexBuffer.SetMaxBoneInfluences(MaxBoneInfluences);
	OutLODRenderData.SkinWeightVertexBuffer.SetUse16BitBoneIndex(bUse16BitBoneIndex);
	OutLODRenderData.SkinWeightVertexBuffer.SetUse16BitBoneWeight(bUse16BitBoneWeight);
	OutLODRenderData.SkinWeightVertexBuffer = SkinWeights;  // Assigns the skinweights

	// Skinweight profiles
	OutLODRenderData.SkinWeightProfilesData.Init(&OutLODRenderData.SkinWeightVertexBuffer);

	// TODO: 
	//     The Cloth Asset doesn't have SkinWeightProfilesData yet, 
	//     but the merging code will have to be implemented once it does.

	// Half edges
	const bool bHasHalfEdges = NumVertices && (
		OutLODRenderData.HalfEdgeBuffer.IsCPUDataValid() ||
		LODRenderData.HalfEdgeBuffer.IsCPUDataValid());

	if (bHasHalfEdges)
	{
		OutLODRenderData.HalfEdgeBuffer.CleanUp();
		OutLODRenderData.HalfEdgeBuffer.Init(OutLODRenderData);
	}

	// Merge cloth data
	const bool bHasClothingData = NumVertices && (
		OutLODRenderData.ClothVertexBuffer.GetNumVertices() != 0 ||  // Note GetNumVertices() doesn't return NumVertices because
		LODRenderData.ClothVertexBuffer.GetNumVertices() != 0);      // of the LODBias there could be several mappings per vertex

	if (bHasClothingData)
	{
		TArray<FMeshToMeshVertData> ClothMappingData = GetClothMappingData(OutLODRenderData.RenderSections);
		ClothMappingData.Append(GetClothMappingData(LODRenderData.RenderSections, &AssetGuids));

		if (ClothMappingData.Num())  // Make sure the sections that have mappings haven't been filtered since ClothVertexBuffer.Init doesn't take empty mapping without crashing
		{
			TArray<FClothBufferIndexMapping> ClothIndexMappings =
				GetClothBufferIndexMappings(OutLODRenderData.ClothVertexBuffer, OutLODRenderData.RenderSections);
			ClothIndexMappings.Append(
				GetClothBufferIndexMappings(LODRenderData.ClothVertexBuffer, LODRenderData.RenderSections, VertexOffset, &AssetGuids));

			OutLODRenderData.ClothVertexBuffer.Init(ClothMappingData, ClothIndexMappings);
		}
	}

	// Merge sections
	const int32 SectionOffset = OutLODRenderData.RenderSections.Num();  // The index of the first merged section
	const int32 NumSections = SectionOffset + LODRenderData.RenderSections.Num();
	OutLODRenderData.RenderSections.Reserve(NumSections);

	for (const FSkelMeshRenderSection& RenderSection : LODRenderData.RenderSections)
	{
		const int32 PieceIndex = AssetGuids.Find(RenderSection.ClothingData.AssetGuid);
		if (PieceIndex != INDEX_NONE || !RenderSection.ClothingData.AssetGuid.IsValid())
		{
			FSkelMeshRenderSection& OutRenderSection = OutLODRenderData.RenderSections.AddDefaulted_GetRef();

			OutRenderSection.MaterialIndex = RenderSection.MaterialIndex + MaterialOffset;
			OutRenderSection.BaseIndex = RenderSection.BaseIndex + IndexOffset;
			OutRenderSection.NumTriangles = RenderSection.NumTriangles;
			OutRenderSection.bRecomputeTangent = RenderSection.bRecomputeTangent;
			OutRenderSection.bCastShadow = RenderSection.bCastShadow;
			OutRenderSection.bVisibleInRayTracing = RenderSection.bVisibleInRayTracing;
			OutRenderSection.RecomputeTangentsVertexMaskChannel = RenderSection.RecomputeTangentsVertexMaskChannel;
			OutRenderSection.BaseVertexIndex = RenderSection.BaseVertexIndex + VertexOffset;
			OutRenderSection.ClothMappingDataLODs = RenderSection.ClothMappingDataLODs;
			OutRenderSection.BoneMap = RenderSection.BoneMap;
			OutRenderSection.NumVertices = RenderSection.NumVertices;
			OutRenderSection.MaxBoneInfluences = RenderSection.MaxBoneInfluences;
			OutRenderSection.CorrespondClothAssetIndex = PieceIndex;
			OutRenderSection.ClothingData = RenderSection.ClothingData;
			OutRenderSection.DuplicatedVerticesBuffer.Init(0, TMap<int32, TArray<int32>>());
			OutRenderSection.bDisabled = RenderSection.bDisabled;

			for (FBoneIndexType& BoneIndex : OutRenderSection.BoneMap)
			{
				BoneIndex = BoneMap[BoneIndex];
			}
		}
	}

	// Update used bone indices
	MergeBones(ReferenceSkeleton, BoneMap, LODRenderData.ActiveBoneIndices, OutLODRenderData.ActiveBoneIndices);
	MergeBones(ReferenceSkeleton, BoneMap, LODRenderData.RequiredBones, OutLODRenderData.RequiredBones);
}

template<typename InRenderDataType, typename OutRenderDataType, typename PiecesType>
void UChaosOutfit::Merge(
	const FReferenceSkeleton& InReferenceSkeleton,
	const InRenderDataType* const InSkeletalMeshRenderData,
	const TArray<FSkeletalMaterial>& InMaterials,
	const FManagedArrayCollection& InOutfitCollection,
	const PiecesType& Pieces,
	const int32 NumPieces,
	const FString& BodySizeNameFilter,
	TArray<FChaosOutfitPiece>& OutPieces,
	FReferenceSkeleton& OutReferenceSkeleton,
	OutRenderDataType* const OutSkeletalMeshRenderData,
	TArray<FSkeletalMaterial>& OutMaterials,
	FManagedArrayCollection& OutOutfitCollection)
{
	using namespace UE::Chaos::OutfitAsset;

	// Filter the body size pieces to merge, or select all pieces
	TArray<FGuid> AssetGuidsFilter;

	const FCollectionOutfitConstFacade InOutfitFacade(InOutfitCollection);
	FCollectionOutfitFacade OutOutfitFacade(OutOutfitCollection);
	check(InOutfitFacade.IsValid());
	check(OutOutfitFacade.IsValid());
	
	if (!BodySizeNameFilter.IsEmpty())
	{
		// Filter to the requested body size
		const int32 BodySize = InOutfitFacade.FindBodySize(BodySizeNameFilter);
		if (BodySize == INDEX_NONE)
		{
			UE_LOGF(LogChaosOutfitAsset, Display, "The requested body size [%ls] isn't available to merge.", *BodySizeNameFilter);
			return;
		}

		// Find all pieces assigned to this body size
		TMap<FGuid, FString> OutfitPieces;
		for (const FGuid& Guid : InOutfitFacade.GetOutfitGuids())
		{
			OutfitPieces.Append(InOutfitFacade.GetOutfitPieces(Guid, BodySize));
		}
		OutfitPieces.GetKeys(AssetGuidsFilter);

		// Merge collection, but only the requested body size
		OutOutfitFacade.Append(InOutfitFacade, BodySize);
	}
	else
	{
		// Add all pieces to the merge
		AssetGuidsFilter = InOutfitFacade.GetOutfitPiecesGuids();

		// Merge the entire collection
		OutOutfitFacade.Append(InOutfitFacade);
	}

	// Merge skeletons
	TArray<int32> BoneMap;
	MergeSkeletons(InReferenceSkeleton, OutReferenceSkeleton, BoneMap);

	// Merge pieces
	TArray<FGuid> AssetGuids;
	AssetGuids.Reserve(NumPieces);
	OutPieces.Reserve(OutPieces.Num() + NumPieces);

	for (int32 PieceIndex = 0; PieceIndex < NumPieces; ++PieceIndex)
	{
		if (AssetGuidsFilter.Contains(Private::GetAssetGuid(Pieces, PieceIndex)))
		{
			FChaosOutfitPiece& Piece = OutPieces.Add_GetRef(Private::Clone(Pieces, PieceIndex));
			Piece.RemapBoneIndices(BoneMap);
			// TODO: Fix case where a piece already exists in the outfit as to not mess up the LODRenderData piece indices
			UE_CLOGF(AssetGuids.Contains(Piece.AssetGuid), LogChaosOutfitAsset, Warning, "Piece [%ls] already exists in this Outfit!", *Piece.Name.ToString());
			AssetGuids.Add(Piece.AssetGuid);
		}
	}

	// Merge materials
	const int32 MaterialOffset = OutMaterials.Num();
	OutMaterials.Append(InMaterials);  // TODO: Merge duplicate material slots. But when doing so, don't forget to consider and update the fix up code in GetOutiftClothCollectionsNode.cpp.

	// Merge render data, only for existing LODs
	// Note: This does not create LODs for the added sections when they are missing
	//       as it would require to duplicate data and add new LODBias clothing data.
	if (InSkeletalMeshRenderData)
	{
		using InLODRenderDataType = decltype(InSkeletalMeshRenderData->LODRenderData);
		using OutLODRenderDataType = decltype(OutSkeletalMeshRenderData->LODRenderData);
		using LODRenderDataElementType = typename OutLODRenderDataType::ElementType;

		const InLODRenderDataType& InLODRenderData = InSkeletalMeshRenderData->LODRenderData;
		OutLODRenderDataType& LODRenderData = OutSkeletalMeshRenderData->LODRenderData;

		const int32 NumLODs = InLODRenderData.Num();
		LODRenderData.Reserve(NumLODs);

		for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
		{
			if (LODIndex >= LODRenderData.Num())
			{
				LODRenderData.Add(new LODRenderDataElementType());
			}
		}

		if (NumLODs > 1)
		{

			TArray<UE::Tasks::FTask> PendingTasks;
			PendingTasks.Reserve(NumLODs);
			for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
			{
				UE::Tasks::FTask PendingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [LODIndex , &InLODRenderData, &AssetGuids, MaterialOffset, &OutReferenceSkeleton, &BoneMap, &LODRenderData]()
				{
					MergeLODRenderDatas(
						InLODRenderData[LODIndex],
						AssetGuids,
						MaterialOffset,
						OutReferenceSkeleton,
						BoneMap,
						LODRenderData[LODIndex]);
				});
				PendingTasks.Add(PendingTask);
			}
			UE::Tasks::Wait(PendingTasks);
		}
		else if (NumLODs == 1)
		{
			const int32 LODIndex = 0;

			MergeLODRenderDatas(
				InLODRenderData[LODIndex],
				AssetGuids,
				MaterialOffset,
				OutReferenceSkeleton,
				BoneMap,
				LODRenderData[LODIndex]);
		}

		OutSkeletalMeshRenderData->NumInlinedLODs = NumLODs;
		OutSkeletalMeshRenderData->NumNonOptionalLODs = NumLODs;
		OutSkeletalMeshRenderData->bSupportRayTracing = OutSkeletalMeshRenderData->bSupportRayTracing || InSkeletalMeshRenderData->bSupportRayTracing;
	}
}

#undef LOCTEXT_NAMESPACE
