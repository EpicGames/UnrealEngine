// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosOutfitAsset/OutfitAsset.h"
#if WITH_EDITOR
#include "ChaosClothAsset/ClothEngineTools.h"
#endif
#include "ChaosClothAsset/ClothSimulationModel.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosOutfitAsset/CollectionOutfitFacade.h"
#include "ChaosOutfitAsset/OutfitAssetPrivate.h"
#include "Dataflow/DataflowContextAssetStore.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetAsyncCompileUtils.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "Rendering/SkeletalMeshRenderData.h"
#if WITH_EDITOR
#include "ChaosOutfitAsset/OutfitAssetBuilder.h"
#include "DerivedDataCacheInterface.h"
#include "Interfaces/ITargetPlatform.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshModel.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(OutfitAsset)

// If Chaos outfit asset derived data needs to be rebuilt (new format, serialization differences, etc.) replace the version GUID below with a new one.
// In case of merge conflicts with DDC versions, you MUST generate a new GUID and set this new GUID as the version.
#define UE_CHAOS_OUTFIT_ASSET_DERIVED_DATA_VERSION TEXT("38712B21FA2B4B3EBC580C4542AC8AC0")

UChaosOutfitAsset::UChaosOutfitAsset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	DataflowInstance.SetDataflowTerminal(TEXT("OutfitAssetTerminal"));
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Templates/CDOs are never rendered and shouldn't run the asset Build path.
	// Base class construction already sets up LODInfo[0] and the default reference skeleton,
	// so a template has the minimal valid state without our Build.
	if (!IsTemplate())
	{
		// Init an empty asset with a root bone and empty placeholder render data.
		// When this object is being loaded, defer resource init: Serialize may run off the game thread
		// (async cooked package load) and will replace the placeholder render data, and releasing GPU
		// resources off-thread is illegal. FinishPostLoadInternal will call InitResources on the game
		// thread once the real data is in place. For fresh NewObject paths, init now so the placeholder
		// has valid (empty) GPU buffers and the asset can be safely bound to a component.
		const bool bDeferResourceInit = HasAnyFlags(RF_NeedLoad);
		Build(nullptr, nullptr, bDeferResourceInit);
	}
}

UChaosOutfitAsset::UChaosOutfitAsset(FVTableHelper& Helper)
	: Super(Helper)
{
}

UChaosOutfitAsset::~UChaosOutfitAsset() = default;

bool UChaosOutfitAsset::HasValidClothSimulationModels() const
{
	for (const FChaosOutfitPiece& Piece : Pieces)
	{
		if (Piece.ClothSimulationModel->GetNumLods())
		{
			return true;
		}
	}
	return false;
}

void UChaosOutfitAsset::Build(const TObjectPtr<const UChaosOutfit> InOutfit, UE::Dataflow::IContextAssetStoreInterface* ContextAssetStore, bool bDeferResourceInit)
{
	using namespace UE::Chaos::OutfitAsset;

	FAutoScopedDurationTimer BuildTimer;
	if (InOutfit)
	{
		UE_LOGF(LogChaosOutfitAsset, Display,
			"[%ls] Build start: InOutfit=%ls Pieces=%d.",
			*GetName(),
			*InOutfit->GetName(),
			InOutfit->GetPieces().Num());
	}
	else
	{
		// Verbose: this fires from the constructor's empty-placeholder init for every loaded outfit asset; logging at Display would spam every load.
		UE_LOGF(LogChaosOutfitAsset, Verbose, "[%ls] Build start: empty placeholder.", *GetName());
	}

	// Unregister dependent components, the context will reregister them at the end of the scope
	const FMultiComponentReregisterContext MultiComponentReregisterContext(GetDependentComponents());

	// Stop the rendering
	ReleaseResources();

	// Copy the outfit to this asset
	TUniquePtr<FSkeletalMeshRenderData> TempSkeletalMeshRenderData = MakeUnique<FSkeletalMeshRenderData>();
	FReferenceSkeleton TempReferenceSkeleton;

	if (InOutfit)
	{
		InOutfit->CopyTo(
			Pieces,
			TempReferenceSkeleton,
			TempSkeletalMeshRenderData,
			Materials,
			OutfitCollection);

#if WITH_EDITORONLY_DATA
		if (Outfit != InOutfit)
		{
			const FName UniqueOutfitName = MakeUniqueObjectName(this, UChaosOutfit::StaticClass());
			Outfit = DuplicateObject<UChaosOutfit>(InOutfit, this, UniqueOutfitName);
		}
#endif
		if (ContextAssetStore)
		{
			// Fix up resized material paths
			TMap<FSoftObjectPath, FSoftObjectPath> MaterialPathsToFixUp;
			MaterialPathsToFixUp.Reserve(Materials.Num());

			for (int32 MaterialIndex = 0; MaterialIndex < Materials.Num(); ++MaterialIndex)
			{
				FSkeletalMaterial& Material = Materials[MaterialIndex];
				if (Material.MaterialInterface && Material.MaterialInterface->GetOuter() == GetTransientPackage())
				{
					const FString TransientPathName = Material.MaterialInterface->GetPathName();
					Material.MaterialInterface = Cast<UMaterialInterface>(ContextAssetStore->CommitAsset(TransientPathName));
#if WITH_EDITORONLY_DATA
					if (Outfit)
					{
						Outfit->GetMaterials()[MaterialIndex].MaterialInterface = Material.MaterialInterface;
					}
#endif
					MaterialPathsToFixUp.Emplace(TransientPathName, Material.MaterialInterface ? Material.MaterialInterface->GetPathName() : FString());
				}
			}

			// Fix up cloth collections
			if (MaterialPathsToFixUp.Num())
			{
				auto FixUpPiecesMaterials = [&MaterialPathsToFixUp](TArrayView<FChaosOutfitPiece> PiecesToFixUp)
					{
						using namespace UE::Chaos::ClothAsset;
						for (FChaosOutfitPiece& Piece : PiecesToFixUp)
						{
							for (TSharedRef<const FManagedArrayCollection>& Collection : Piece.Collections)
							{
								TOptional<TSharedRef<FManagedArrayCollection>> FixedUpCollection;
								TOptional<FCollectionClothFacade> FixedUpClothFacade;

								FCollectionClothConstFacade ClothFacade(Collection);
								const TConstArrayView<FSoftObjectPath> RenderMaterialPathNames = ClothFacade.GetRenderMaterialSoftObjectPathName();
								for (int32 PathIndex = 0; PathIndex < RenderMaterialPathNames.Num(); ++PathIndex)
								{
									const FSoftObjectPath& RenderMaterialPathName = RenderMaterialPathNames[PathIndex];
									if (const FSoftObjectPath* MaterialPathToFixUp = MaterialPathsToFixUp.Find(RenderMaterialPathName))
									{
										if (!FixedUpCollection.IsSet())
										{
											FixedUpCollection.Emplace(MakeShared<FManagedArrayCollection>(*Collection));
											FixedUpClothFacade.Emplace(FixedUpCollection.GetValue());
										}
										TArrayView<FSoftObjectPath> FixedUpRenderMaterialPathNames = FixedUpClothFacade->GetRenderMaterialSoftObjectPathName();
										check(FixedUpRenderMaterialPathNames[PathIndex] == RenderMaterialPathName);
							
										FixedUpRenderMaterialPathNames[PathIndex] = *MaterialPathToFixUp;
									}
								}

								if (FixedUpCollection.IsSet())
								{
									Collection = MoveTemp(FixedUpCollection.GetValue());
								}
							}
						}
					};

				// Fix up this outfit's pieces
				FixUpPiecesMaterials(Pieces);

#if WITH_EDITORONLY_DATA
				// Fix up this outfit construction's pieces
				if (Outfit)
				{
					FixUpPiecesMaterials(Outfit->GetPieces());
				}
#endif
			}
		}
	}
	else
	{
		UChaosOutfit::Init(
			Pieces,
			TempReferenceSkeleton,
			TempSkeletalMeshRenderData,
			Materials,
			OutfitCollection);

#if WITH_EDITORONLY_DATA
		Outfit = nullptr;
#endif
	}

	// Populate the body sizes
	const FCollectionOutfitConstFacade OutfitFacade(OutfitCollection);
	const TArray<FSoftObjectPath> OutfitBodyPartsSkeletalMeshes = OutfitFacade.GetOutfitBodyPartsSkeletalMeshPaths();
	Bodies.Reset(OutfitBodyPartsSkeletalMeshes.Num());
	for (const FSoftObjectPath& OutfitBodyPartsSkeletalMesh : OutfitBodyPartsSkeletalMeshes)
	{
		if (USkeletalMesh* const Body = Cast<USkeletalMesh>(OutfitBodyPartsSkeletalMesh.TryLoad()))
		{
			Bodies.Emplace(Body);
		}
	}

	// Set the new reference skeleton
	SetReferenceSkeleton(&TempReferenceSkeleton);

	// Set LODInfo from the outfit data
	const int32 NumLODs = GetNumLODsFromPieces();
	LODInfo.Reset(NumLODs);
	LODInfo.AddDefaulted(NumLODs);

	// Set render data from the merge result
	if (!TempSkeletalMeshRenderData->LODRenderData.Num())
	{
		TempSkeletalMeshRenderData->LODRenderData.Add(new FSkeletalMeshLODRenderData());
		TempSkeletalMeshRenderData->LODRenderData[0].StaticVertexBuffers.PositionVertexBuffer.Init(0);  // Required for serialization
		TempSkeletalMeshRenderData->LODRenderData[0].StaticVertexBuffers.StaticMeshVertexBuffer.Init(0, 0);  // Required for serialization
	}

	ReleaseResourcesFence.Wait();  // Make sure the release resources fence has completed before deleting/replacing the render data
	SetResourceForRendering(MoveTemp(TempSkeletalMeshRenderData));

	CalculateBounds();

#if WITH_EDITOR
	// Store to DDC for future loads
	if (InOutfit)
	{
		StoreDerivedData();
	}
#endif

	if (!bDeferResourceInit)
	{
		if (FApp::CanEverRender())
		{
			InitResources();
		}
		else
		{
			UpdateUVChannelData(false);
		}

		CalculateInvRefMatrices();
	}

	if (InOutfit)
	{
		UE_LOGF(LogChaosOutfitAsset, Display, "[%ls] Build complete in %.1f ms.", *GetName(), BuildTimer.GetTime() * 1000.0);
	}
	else
	{
		UE_LOGF(LogChaosOutfitAsset, Verbose, "[%ls] Build complete (empty placeholder) in %.1f ms.", *GetName(), BuildTimer.GetTime() * 1000.0);
	}

	// Update any components using this asset
	constexpr bool bReregisterComponents = false;  // Do not reregister twice, this is already done at the function scope
	OnAssetChanged(bReregisterComponents);
}

void UChaosOutfitAsset::Serialize(FArchive& Ar)
{
	LLM_SCOPE_BYNAME(TEXT("Physics/Cloth"));
	Super::Serialize(Ar);

	bool bCooked = Ar.IsCooking();

#if WITH_EDITORONLY_DATA
	// Outfit is used to gate legacy asset version, before the asset was cooked and serialized its render data payload.
	// Ar.IsLoadingFromCookedPackage() allows cooked for editor workflows, to reload assets with a cooked (stripped) nullptr Outfit.
	// bCooked makes sure that the cooking serialization never skip bCooked, this matches legacy behavior.
	if (Outfit || Ar.IsLoadingFromCookedPackage() || bCooked)
#endif
	{
		Ar << bCooked;

		if (bCooked && !IsTemplate() && !Ar.IsCountingMemory())  // Counting of these resources are done in GetResourceSizeEx, so skip these when counting memory
		{
			LLM_SCOPE_BYNAME(TEXT("Physics/ClothRendering"));
			if (Ar.IsLoading())
			{
				// The constructor uses bDeferResourceInit so the placeholder render data has no GPU resources.
				// We can replace it directly here, even when Serialize runs off the game thread (async load).
				SetResourceForRendering(MakeUnique<FSkeletalMeshRenderData>());
				GetResourceForRendering()->Serialize(Ar, this);
#if WITH_EDITOR
				EstablishCookedHeadKey();
#else
				UE_LOGF(LogChaosOutfitAsset, Display, "[%ls] Serialize: cooked load.", *GetName());
#endif
			}
			else if (Ar.IsSaving())
			{
#if WITH_EDITOR
				if (FSkeletalMeshRenderData* const LocalRenderData = GetSerializeRenderData(Ar))
				{
					LocalRenderData->Serialize(Ar, this);
				}
				else
				{
					return;  // SetError already called inside GetSerializeRenderData
				}
#else
				GetResourceForRendering()->Serialize(Ar, this);
#endif
			}
		}
	}
	if (Ar.IsLoading())
	{
		UE::Chaos::OutfitAsset::FCollectionOutfitFacade OutfitFacade(OutfitCollection);
		OutfitFacade.PostSerialize(Ar);
	}
}

void UChaosOutfitAsset::PostLoad()
{
	LLM_SCOPE_BYNAME(TEXT("Physics/Cloth"));
	// Super::PostLoad drives the BeginPostLoadInternal/ExecutePostLoadInternal/FinishPostLoadInternal chain
	Super::PostLoad();
}

#if WITH_EDITOR
const TCHAR* UChaosOutfitAsset::GetDerivedDataVersion() const
{
	return UE_CHAOS_OUTFIT_ASSET_DERIVED_DATA_VERSION;
}

void UChaosOutfitAsset::BuildLODModel(FSkeletalMeshRenderData& RenderData, const ITargetPlatform* TargetPlatform, int32 LODIndex)
{
	UE_LOGF(LogChaosOutfitAsset, Verbose,
		"[%ls] BuildLODModel: LOD %d for [%ls].",
		*GetName(),
		LODIndex,
		TargetPlatform ? *TargetPlatform->PlatformName() : TEXT("(null)"));
	// RenderData is the target being built by Cache(), we don't write to it directly.
	// Instead, we populate the MeshModel LOD from the running-platform render data;
	// Cache() then converts the LODModel into platform-specific RenderData.
	check(GetImportedModel() && GetImportedModel()->LODModels.IsValidIndex(LODIndex));
	const FSkeletalMeshRenderData* const SourceRenderData = GetResourceForRendering();
	check(SourceRenderData && SourceRenderData->LODRenderData.IsValidIndex(LODIndex));
	FBuilder::BuildLod(
		GetImportedModel()->LODModels[LODIndex],
		SourceRenderData->LODRenderData[LODIndex],
		TargetPlatform);
}

FString UChaosOutfitAsset::BuildDerivedDataKey(const ITargetPlatform* TargetPlatform)
{
	FString KeySuffix;

	// Include each piece's AssetGuid (captures piece identity and version)
	for (const FChaosOutfitPiece& Piece : Pieces)
	{
		KeySuffix += Piece.AssetGuid.ToString();
	}

	// Number of LODs (stable - derived from pieces, not render data which may not exist at fetch time)
	KeySuffix += TEXT("_");
	KeySuffix += FString::FromInt(GetNumLODsFromPieces());

	// Build settings (GPU bone limits, bone influences, LODInfo GUIDs)
	AppendBuildSettingsToDDCKey(KeySuffix, TargetPlatform);

	return FDerivedDataCacheInterface::BuildCacheKey(
		GetDerivedDataPrefix(),
		GetDerivedDataVersion(),
		*KeySuffix);
}

void UChaosOutfitAsset::BeginPostLoadAssetImpl(FSkinnedAssetPostLoadContext& Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UChaosOutfitAsset::BeginPostLoadAssetImpl);

#if WITH_EDITORONLY_DATA
	if (Outfit)
	{
		// IsInitialBuildDone() returns true when the head render data carries a DerivedDataKey,
		// so the constructor's empty placeholder (no key) is treated as not-built.
		const bool bNeedsRenderData = !GetOutermost()->bIsCookedForEditor && !IsInitialBuildDone();
		if (bNeedsRenderData)
		{
			// Try DDC for running platform render data before the expensive Build/merge.
			// Assembly data (Pieces, Materials, OutfitCollection, etc.) is already deserialized from the package.
			CacheDerivedData(&Context);
			Context.bHasCachedDerivedData = true;

			if (!IsInitialBuildDone())
			{
				// DDC miss: full rebuild including expensive merge. Build() stores result to DDC.
				// FinishPostLoadInternal handles CalculateInvRefMatrices + InitResources.
				constexpr bool bDeferResourceInit = true;
				Build(Outfit, nullptr, bDeferResourceInit);
			}
		}
		else
		{
			UE_LOGF(LogChaosOutfitAsset, Display, "[%ls] PostLoad: skipped (cooked-for-editor or already built).", *GetName());
		}
	}
	else
#endif // WITH_EDITORONLY_DATA
	{
		if (GetDataflow())
		{
			UE_LOGF(LogChaosOutfitAsset, Display, "[%ls] PostLoad: legacy path (no cached Outfit), re-evaluating Dataflow.", *GetName());
			// Legacy PostLoad: re-evaluate the Dataflow
			if (GetDataflowInstance().UpdateOwnerAsset())
			{
				if (UPackage* const Package = GetOutermost())
				{
					Package->SetDirtyFlag(true);
				}
				UE_LOGF(LogChaosOutfitAsset, Warning, "Outfit Asset [%ls] needs to be re-saved.", *GetName());
			}
		}

		// For legacy assets without source data, try DDC fetch; if that misses too, the asset is left
		// unbuilt and GetPlatformSkeletalMeshRenderData will short-circuit to the empty head per-platform
		if (!IsInitialBuildDone() && !GetOutermost()->bIsCookedForEditor)
		{
			CacheDerivedData(&Context);
			Context.bHasCachedDerivedData = true;
			if (!IsInitialBuildDone())
			{
				UE_LOGF(LogChaosOutfitAsset, Error,
					"[%ls] PostLoad: no Outfit, no Dataflow, no DDC entry; asset is unbuilt and cannot produce render data.",
					*GetName());
			}
		}
	}
}
#endif // WITH_EDITOR

FName UChaosOutfitAsset::GetClothSimulationModelName(int32 ModelIndex) const
{
	return Pieces[ModelIndex].Name;
}

TSharedPtr<const FChaosClothSimulationModel> UChaosOutfitAsset::GetClothSimulationModel(int32 ModelIndex) const
{
	return Pieces[ModelIndex].ClothSimulationModel;
}

const TArray<TSharedRef<const FManagedArrayCollection>>& UChaosOutfitAsset::GetCollections(int32 ModelIndex) const
{
	return Pieces[ModelIndex].Collections;
}

const UPhysicsAsset* UChaosOutfitAsset::GetPhysicsAssetForModel(int32 ModelIndex) const
{
	return Pieces[ModelIndex].PhysicsAsset;
}

FGuid UChaosOutfitAsset::GetAssetGuid(int32 ModelIndex) const
{
	return Pieces[ModelIndex].AssetGuid;
}

int32 UChaosOutfitAsset::GetNumLODsFromPieces() const
{
	int32 NumLODs = 0;
	for (const FChaosOutfitPiece& Piece : Pieces)
	{
		NumLODs = FMath::Max(Piece.Collections.Num(), NumLODs);
	}
	return FMath::Max(NumLODs, 1);
}

#if WITH_EDITOR
UChaosClothAssetBase* UChaosOutfitAsset::CreatePreviewAssetCopyImpl(
	UObject* Outer,
	EObjectFlags Flags,
	bool bFilterToSingleSize,
	bool bBuildSimModelRenderData) const
{
	using namespace UE::Chaos::ClothAsset;
	using namespace UE::Chaos::OutfitAsset;

	UChaosOutfitAsset* PreviewAsset = nullptr;

	// Filter to a single body size for multi-size outfits
	const FCollectionOutfitConstFacade OutfitFacade(GetOutfitCollection());
	if (bFilterToSingleSize && OutfitFacade.IsValid() && OutfitFacade.GetNumBodySizes() >= 2)
	{
		const FString& SizeName = OutfitFacade.GetBodySizeName(0);
		if (!SizeName.IsEmpty())
		{
			UChaosOutfit* const FullOutfit = NewObject<UChaosOutfit>();
			FullOutfit->Add(*this);

			UChaosOutfit* const FilteredOutfit = NewObject<UChaosOutfit>();
			FilteredOutfit->Append(*FullOutfit, SizeName);

			PreviewAsset = NewObject<UChaosOutfitAsset>(Outer, StaticClass(), NAME_None, Flags);
			PreviewAsset->Build(FilteredOutfit);
		}
	}

	if (!bBuildSimModelRenderData)
	{
		return PreviewAsset;
	}

	// Build a fresh sim-model preview when the asset to preview has no render data
	const UChaosClothAssetBase* const AssetToPreview = PreviewAsset ? PreviewAsset : this;
	FSkeletalMeshRenderData* const RenderData = AssetToPreview->GetResourceForRendering();
	const bool bHasRenderData = RenderData &&
		RenderData->LODRenderData.Num() &&
		RenderData->LODRenderData[0].GetTotalFaces();

	if (!bHasRenderData && AssetToPreview->HasValidClothSimulationModels())
	{
		if (UChaosClothAssetBase* const SimPreview = CreateSimModelPreviewAsset(
			*AssetToPreview, Outer, Flags, FClothEngineTools::GetSimPreviewMaterial()))
		{
			PreviewAsset = CastChecked<UChaosOutfitAsset>(SimPreview);
		}
	}

	return PreviewAsset;
}
#endif
