// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "ChaosOutfitAsset/OutfitAssetBuilder.h"
#include "ChaosOutfitAsset/OutfitAssetPrivate.h"
#include "ChaosOutfitAsset/OutfitAssetUtility.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Rendering/SkinWeightVertexBuffer.h"
#include "GPUSkinVertexFactory.h"
#include "MeshBuild.h"

void UChaosOutfitAsset::FBuilder::BuildLod(
	FSkeletalMeshLODModel& LODModel,
	const FSkeletalMeshLODRenderData& SourceLODRenderData,
	const ITargetPlatform* TargetPlatform)
{
	using namespace UE::Chaos::OutfitAsset;

	check(TargetPlatform);

	LODModel.Empty();
	LODModel.MaxImportVertex = 0;
	LODModel.NumTexCoords = SourceLODRenderData.StaticVertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();
	LODModel.NumVertices = 0;

	const TArray<FSkelMeshRenderSection>& SourceSections = SourceLODRenderData.RenderSections;
	const int32 NumSections = SourceSections.Num();
	if (NumSections == 0)
	{
		return;
	}

	// Validate vertex buffers before reading
	const uint32 TotalVertices = SourceLODRenderData.GetNumVertices();
	const FPositionVertexBuffer& PositionBuffer = SourceLODRenderData.StaticVertexBuffers.PositionVertexBuffer;
	const FStaticMeshVertexBuffer& VertexBuffer = SourceLODRenderData.StaticVertexBuffers.StaticMeshVertexBuffer;
	const FColorVertexBuffer& ColorBuffer = SourceLODRenderData.StaticVertexBuffers.ColorVertexBuffer;
	const FSkinWeightVertexBuffer& SkinWeightBuffer = SourceLODRenderData.SkinWeightVertexBuffer;
	if (TotalVertices == 0 ||
		PositionBuffer.GetNumVertices() != TotalVertices ||
		VertexBuffer.GetNumVertices() != TotalVertices ||
		SkinWeightBuffer.GetNumVertices() != TotalVertices)
	{
		UE_LOGF(
			LogChaosOutfitAsset,
			Warning,
			"OutfitAsset BuildLod: skipping LOD with inconsistent vertex buffers (Total=%u, Pos=%u, Vert=%u, Skin=%u).",
			TotalVertices,
			PositionBuffer.GetNumVertices(),
			VertexBuffer.GetNumVertices(),
			SkinWeightBuffer.GetNumVertices());
		return;
	}

	// Platform-specific bone influence limits
	const int32 MaxBoneInfluencesFromUnlimitedBoneInfluences =
		FGPUBaseSkinVertexFactory::GetUnlimitedBoneInfluences(TargetPlatform) ? MAX_TOTAL_INFLUENCES : EXTRA_BONE_INFLUENCES;
	const int32 MaxBoneInfluencesFromPlatformProjectSettings =
		FGPUBaseSkinVertexFactory::GetBoneInfluenceLimitForAsset(0, TargetPlatform);
	const int32 MaxNumInfluences = FMath::Min(MaxBoneInfluencesFromUnlimitedBoneInfluences, MaxBoneInfluencesFromPlatformProjectSettings);

	// Read the global index buffer
	TArray<uint32> SourceIndices = GetIndices(SourceLODRenderData.MultiSizeIndexContainer);
	if (SourceIndices.IsEmpty())
	{
		UE_LOGF(LogChaosOutfitAsset, Warning, "OutfitAsset BuildLod: skipping LOD with empty index buffer.");
		return;
	}

	// Vertex colors are optional; only consume them when the buffer is sized to match the geometry.
	const bool bSourceHasVertexColors =
		ColorBuffer.GetAllocatedSize() != 0 &&
		ColorBuffer.GetNumVertices() == TotalVertices;
	const uint32 NumTexCoords = VertexBuffer.GetNumTexCoords();
	const uint32 SourceMaxBoneInfluences = SkinWeightBuffer.GetMaxBoneInfluences();

	// Build sections
	LODModel.Sections.SetNum(NumSections);

	for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
	{
		const FSkelMeshRenderSection& SourceSection = SourceSections[SectionIndex];
		FSkelMeshSection& OutSection = LODModel.Sections[SectionIndex];

		OutSection.OriginalDataSectionIndex = SectionIndex;
		OutSection.MaterialIndex = SourceSection.MaterialIndex;
		OutSection.NumTriangles = SourceSection.NumTriangles;
		OutSection.BaseIndex = SourceSection.BaseIndex;
		OutSection.BaseVertexIndex = SourceSection.BaseVertexIndex;
		OutSection.NumVertices = SourceSection.NumVertices;
		OutSection.bRecomputeTangent = SourceSection.bRecomputeTangent;
		OutSection.RecomputeTangentsVertexMaskChannel = SourceSection.RecomputeTangentsVertexMaskChannel;
		OutSection.bCastShadow = SourceSection.bCastShadow;
		OutSection.bVisibleInRayTracing = SourceSection.bVisibleInRayTracing;
		OutSection.bDisabled = SourceSection.bDisabled;
		OutSection.CorrespondClothAssetIndex = SourceSection.CorrespondClothAssetIndex;
		OutSection.ClothingData = SourceSection.ClothingData;
		OutSection.ClothMappingDataLODs = SourceSection.ClothMappingDataLODs;

		// Copy BoneMap directly (already in merged skeleton space)
		OutSection.BoneMap = SourceSection.BoneMap;

		// Reconstruct FSoftSkinVertex array from vertex buffers
		const uint32 SectionBaseVertex = SourceSection.BaseVertexIndex;
		const int32 SectionNumVertices = SourceSection.NumVertices;
		OutSection.SoftVertices.SetNumUninitialized(SectionNumVertices);

		for (int32 VertexIndex = 0; VertexIndex < SectionNumVertices; ++VertexIndex)
		{
			const uint32 GlobalVertexIndex = SectionBaseVertex + VertexIndex;
			FSoftSkinVertex& SoftVertex = OutSection.SoftVertices[VertexIndex];

			// Position
			SoftVertex.Position = PositionBuffer.VertexPosition(GlobalVertexIndex);

			// Tangents (TangentX is vec3, TangentY is vec3, TangentZ is vec4 with determinant sign in W)
			const FVector4f TangentX = VertexBuffer.VertexTangentX(GlobalVertexIndex);
			const FVector4f TangentZ = VertexBuffer.VertexTangentZ(GlobalVertexIndex);
			SoftVertex.TangentX = FVector3f(TangentX.X, TangentX.Y, TangentX.Z);
			SoftVertex.TangentZ = TangentZ;
			// Reconstruct TangentY from X cross Z, with sign from TangentZ.W
			const FVector3f TangentZVec(TangentZ.X, TangentZ.Y, TangentZ.Z);
			SoftVertex.TangentY = FVector3f::CrossProduct(TangentZVec, SoftVertex.TangentX) * TangentZ.W;

			// UVs
			FMemory::Memzero(SoftVertex.UVs, sizeof(SoftVertex.UVs));
			for (uint32 UVIndex = 0; UVIndex < FMath::Min(NumTexCoords, (uint32)MAX_TEXCOORDS); ++UVIndex)
			{
				SoftVertex.UVs[UVIndex] = VertexBuffer.GetVertexUV(GlobalVertexIndex, UVIndex);
			}

			// Color
			SoftVertex.Color = bSourceHasVertexColors ? ColorBuffer.VertexColor(GlobalVertexIndex) : FColor::White;

			// Bone influences read from skin weight buffer (section-local indices into BoneMap),
			// capped at MAX_TOTAL_INFLUENCES to bound writes into FSoftSkinVertex's fixed-size arrays
			int32 NumNonZeroInfluences = 0;
			const uint32 SourceInfluenceLoopLimit = FMath::Min(SourceMaxBoneInfluences, (uint32)MAX_TOTAL_INFLUENCES);
			for (uint32 InfluenceIndex = 0; InfluenceIndex < SourceInfluenceLoopLimit; ++InfluenceIndex)
			{
				const uint16 BoneIndex = (uint16)SkinWeightBuffer.GetBoneIndex(GlobalVertexIndex, InfluenceIndex);
				const uint16 BoneWeight = SkinWeightBuffer.GetBoneWeight(GlobalVertexIndex, InfluenceIndex);
				if (BoneWeight > 0)
				{
					SoftVertex.InfluenceBones[NumNonZeroInfluences] = BoneIndex;
					SoftVertex.InfluenceWeights[NumNonZeroInfluences] = BoneWeight;
					++NumNonZeroInfluences;
				}
			}
			// Zero remaining influence slots
			for (int32 InfluenceIndex = NumNonZeroInfluences; InfluenceIndex < MAX_TOTAL_INFLUENCES; ++InfluenceIndex)
			{
				SoftVertex.InfluenceBones[InfluenceIndex] = 0;
				SoftVertex.InfluenceWeights[InfluenceIndex] = 0;
			}

			// Apply platform bone influence conforming if needed
			if (NumNonZeroInfluences > MaxNumInfluences)
			{
				// Sort influences by weight (descending), keep only MaxNumInfluences
				TArray<TPair<uint16, uint16>, TInlineAllocator<MAX_TOTAL_INFLUENCES>> SortedInfluences;
				SortedInfluences.Reserve(NumNonZeroInfluences);
				for (int32 InfluenceIndex = 0; InfluenceIndex < NumNonZeroInfluences; ++InfluenceIndex)
				{
					SortedInfluences.Emplace(SoftVertex.InfluenceWeights[InfluenceIndex], SoftVertex.InfluenceBones[InfluenceIndex]);
				}
				SortedInfluences.Sort(
					[](const TPair<uint16, uint16>& Left, const TPair<uint16, uint16>& Right)
					{
						return Left.Key > Right.Key;
					});

				// Keep top MaxNumInfluences and renormalize
				uint32 TotalWeight = 0;
				for (int32 InfluenceIndex = 0; InfluenceIndex < MaxNumInfluences; ++InfluenceIndex)
				{
					SoftVertex.InfluenceBones[InfluenceIndex] = SortedInfluences[InfluenceIndex].Value;
					SoftVertex.InfluenceWeights[InfluenceIndex] = SortedInfluences[InfluenceIndex].Key;
					TotalWeight += SoftVertex.InfluenceWeights[InfluenceIndex];
				}
				for (int32 InfluenceIndex = MaxNumInfluences; InfluenceIndex < MAX_TOTAL_INFLUENCES; ++InfluenceIndex)
				{
					SoftVertex.InfluenceBones[InfluenceIndex] = 0;
					SoftVertex.InfluenceWeights[InfluenceIndex] = 0;
				}
				// Redistribute to sum to max weight
				if (TotalWeight > 0 && TotalWeight != TNumericLimits<uint16>::Max())
				{
					const int32 WeightDiff = (int32)TNumericLimits<uint16>::Max() - (int32)TotalWeight;
					SoftVertex.InfluenceWeights[0] = (uint16)FMath::Clamp(
						(int32)SoftVertex.InfluenceWeights[0] + WeightDiff, 0, (int32)TNumericLimits<uint16>::Max());
				}
			}
		}

		OutSection.CalcMaxBoneInfluences();
		OutSection.CalcUse16BitBoneIndex();

		// Build overlapping vertices map
		const TArray<FSoftSkinVertex>& SoftVertices = OutSection.SoftVertices;
		using FVertexProjectionPair = TPair<float, int32>;
		TArray<FVertexProjectionPair> VertexProjectionPairs;
		VertexProjectionPairs.Reserve(SectionNumVertices);

		// Arbitrary non-axis-aligned projection to bucket spatially nearby vertices for overlap detection.
		// The non-uniform coefficients (summing to 1) avoid aligning with any axis or diagonal,
		// reducing collisions when vertices share one or two coordinates.
		constexpr float ProjectionCoefficientX = 0.30f;
		constexpr float ProjectionCoefficientY = 0.33f;
		constexpr float ProjectionCoefficientZ = 0.37f;

		for (int32 VertexIndex = 0; VertexIndex < SectionNumVertices; ++VertexIndex)
		{
			const FVector3f& Position = SoftVertices[VertexIndex].Position;
			const float ProjectedValue =
				ProjectionCoefficientX * Position.X +
				ProjectionCoefficientY * Position.Y +
				ProjectionCoefficientZ * Position.Z;
			VertexProjectionPairs.Emplace(ProjectedValue, VertexIndex);
		}
		VertexProjectionPairs.Sort(
			[](const FVertexProjectionPair& Left, const FVertexProjectionPair& Right)
			{
				return Left.Key < Right.Key;
			});

		for (int32 Index0 = 0; Index0 < VertexProjectionPairs.Num(); ++Index0)
		{
			const float ProjectedValue0 = VertexProjectionPairs[Index0].Key;
			const int32 VertexIndex0 = VertexProjectionPairs[Index0].Value;
			const FVector3f& Position0 = SoftVertices[VertexIndex0].Position;

			for (int32 Index1 = Index0 + 1;
				Index1 < VertexProjectionPairs.Num() && FMath::Abs(VertexProjectionPairs[Index1].Key - ProjectedValue0) <= UE_THRESH_POINTS_ARE_SAME;
				++Index1)
			{
				const int32 VertexIndex1 = VertexProjectionPairs[Index1].Value;
				const FVector3f& Position1 = SoftVertices[VertexIndex1].Position;

				if (PointsEqual(Position0, Position1))
				{
					OutSection.OverlappingVertices.FindOrAdd(VertexIndex0).Add(VertexIndex1);
					OutSection.OverlappingVertices.FindOrAdd(VertexIndex1).Add(VertexIndex0);
				}
			}
		}

		FSkelMeshSourceSectionUserData::GetSourceSectionUserData(LODModel.UserSectionsData, OutSection);

		LODModel.NumVertices += SectionNumVertices;
	}

	// Copy index buffer
	LODModel.IndexBuffer = MoveTemp(SourceIndices);

	// Build MeshToImportVertexMap
	LODModel.MeshToImportVertexMap.SetNumUninitialized(TotalVertices);
	for (uint32 VertexIndex = 0; VertexIndex < TotalVertices; ++VertexIndex)
	{
		LODModel.MeshToImportVertexMap[VertexIndex] = VertexIndex;
	}
	LODModel.MaxImportVertex = (TotalVertices > 0) ? (int32)(TotalVertices - 1) : 0;
	LODModel.NumTexCoords = FMath::Max(LODModel.NumTexCoords, 1u);

	// Copy ActiveBoneIndices and RequiredBones
	LODModel.ActiveBoneIndices = SourceLODRenderData.ActiveBoneIndices;
	LODModel.RequiredBones = SourceLODRenderData.RequiredBones;
}

#endif  // #if WITH_EDITOR
