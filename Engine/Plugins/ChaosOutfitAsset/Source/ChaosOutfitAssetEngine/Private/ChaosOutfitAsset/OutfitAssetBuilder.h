// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "ChaosOutfitAsset/OutfitAsset.h"

class FSkeletalMeshLODModel;
class FSkeletalMeshLODRenderData;

/**
 * Builder utility struct, nested struct of UChaosOutfitAsset.
 * Converts merged FSkeletalMeshLODRenderData into FSkeletalMeshLODModel, applying platform-specific bone influence conforming.
 * This is the reverse of BuildFromLODModel().
 */
struct UChaosOutfitAsset::FBuilder
{
	/** Convert one LOD of merged render data into an FSkeletalMeshLODModel for the specified target platform. */
	static void BuildLod(
		FSkeletalMeshLODModel& OutLODModel,
		const FSkeletalMeshLODRenderData& SourceLODRenderData,
		const ITargetPlatform* TargetPlatform);
};

#endif  // #if WITH_EDITOR
