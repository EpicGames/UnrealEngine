// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Stats/Stats.h"

DECLARE_LOG_CATEGORY_EXTERN(LogChaosOutfitAsset, Log, All);
DECLARE_STATS_GROUP(TEXT("ChaosOutfitAsset"), STATGROUP_ChaosOutfitAsset, STATCAT_Advanced);
