// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosOutfitAsset/OutfitAssetUtility.h"
#include "ChaosLog.h"
#include "ReferenceSkeleton.h"
#include "Rendering/SkeletalMeshVertexClothBuffer.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Rendering/SkinWeightVertexBuffer.h"
#include "Rendering/MultiSizeIndexContainer.h"

namespace UE::Chaos::OutfitAsset
{
	// Merge bone to a new skeleton and return its new index in the new skeleton
	int32 MergeBoneToSkeleton(const int32 BoneIndex, const FReferenceSkeleton& InSkeleton, FReferenceSkeleton& InOutSkeleton)
	{
		const FName BoneName = InSkeleton.GetBoneName(BoneIndex);
		const int32 ParentBoneIndex = InSkeleton.GetParentIndex(BoneIndex);
		const int32 ExistingBoneIndex = InOutSkeleton.FindBoneIndex(BoneName);

		if (ParentBoneIndex == INDEX_NONE)  // Merging root bones
		{
			constexpr int32 RootBoneIndex = 0;  // A root bone always exists at index 0 (actually created in the UChaosOutfit::Init() function)
			const FName ExistingRootBoneName = InOutSkeleton.GetBoneName(RootBoneIndex);

			if (ExistingBoneIndex != INDEX_NONE)  // This root bone has already been merged
			{
				UE_CLOGF(ExistingRootBoneName != BoneName,  // Warn if this bone is not the root in the existing hierarchy
					LogChaos, Warning, "Root bone [%ls] is no longer root bone in the merged hierarchy.", *BoneName.ToString());

				return ExistingBoneIndex;
			}
			else  // The existing root bone has a different name and must be renamed (this also allows for the UChaosOutfit::Init() "Root" name to be changed if required)
			{
				FReferenceSkeletonModifier ReferenceSkeletonModifier(InOutSkeleton, nullptr);
				ReferenceSkeletonModifier.Rename(ExistingRootBoneName, BoneName);

				UE_LOGF(LogChaos, Display, "Root bone name updated from [%ls] to [%ls].", *ExistingRootBoneName.ToString(), *BoneName.ToString());

				return RootBoneIndex;
			}
		}
		else  // Merging non root bones
		{
			const FName ParentBoneName = InSkeleton.GetBoneName(ParentBoneIndex);
			const int32 ExistingParentBoneIndex = InOutSkeleton.FindBoneIndex(ParentBoneName);
			checkf(ExistingParentBoneIndex != INDEX_NONE, TEXT("Parent bones must always be merged first before their children."));

			if (ExistingBoneIndex != INDEX_NONE)  // This bone has already been merged
			{
				UE_CLOG(ExistingParentBoneIndex != InOutSkeleton.GetParentIndex(ExistingBoneIndex),  // Warn if the found bone parent's position doesn't match the original parent bone name's position
					LogChaos, Warning, TEXT("Bone [%s] has already been merged to an Outfit but with a different hierarchy for its parent [%s]."), *BoneName.ToString(), *ParentBoneName.ToString());

				return ExistingBoneIndex;
			}
			else  // This bone hasn't been merged yet, add it as a new bone under its merged parent
			{
				FReferenceSkeletonModifier ReferenceSkeletonModifier(InOutSkeleton, nullptr);

				FMeshBoneInfo MeshBoneInfo;
#if WITH_EDITORONLY_DATA
				MeshBoneInfo.ExportName = BoneName.ToString();
#endif
				MeshBoneInfo.Name = BoneName;
				MeshBoneInfo.ParentIndex = ExistingParentBoneIndex;
				ReferenceSkeletonModifier.Add(MeshBoneInfo, InSkeleton.GetRefBonePose()[BoneIndex]);

				return ReferenceSkeletonModifier.FindBoneIndex(BoneName);
			}
		}
	}

	TArray<int32> GetChildren(const TArray<FMeshBoneInfo>& MeshBoneInfos, const int32 ParentIndex)
	{
		TArray<int32> Children;
		for (int32 Index = 0; Index < MeshBoneInfos.Num(); ++Index)
		{
			const FMeshBoneInfo& MeshBoneInfo = MeshBoneInfos[Index];
			if (MeshBoneInfo.ParentIndex == ParentIndex)
			{
				Children.Emplace(Index);
			}
		}
		return Children;
	}

	void LogHierarchy(const TArray<FMeshBoneInfo>& MeshBoneInfos, const int32 ParentIndex, int32 Indent)
	{
		auto MakeIndentString = [](const int32 Indent) -> FString
			{
				FString IndentString;
				for (int32 Index = 0; Index < Indent; ++Index)
				{
					IndentString += TEXT("   ");
				}
				return IndentString;
			};

		const FString IndentString = MakeIndentString(Indent);

		for (const int32 ChildIndex : GetChildren(MeshBoneInfos, ParentIndex))
		{
			const FString IndentedName = IndentString + MeshBoneInfos[ChildIndex].Name.ToString();

			UE_LOGF(LogChaos, VeryVerbose, "%d - %ls", ChildIndex, *IndentedName);

			LogHierarchy(MeshBoneInfos, ChildIndex, Indent + 1);
		}
	}

	void MergeSkeletons(const FReferenceSkeleton& InSkeleton, FReferenceSkeleton& InOutSkeleton, TArray<int32>& OutBoneMap)
	{
		const int32 NumBones = InSkeleton.GetNum();

		OutBoneMap.SetNumZeroed(NumBones);

		TBitArray BonesToProcess(true, NumBones);

		for (;;)
		{
			// Find the first unprocessed bone
			int32 BoneIndex = BonesToProcess.Find(true);
			if (BoneIndex == INDEX_NONE)
			{
				break;
			}

			// Replace by its parent in case they're not already processed (this assumes the bones aren't sorted to be on the safe side)
			for (int32 ParentBoneIndex = InSkeleton.GetParentIndex(BoneIndex);
				ParentBoneIndex != INDEX_NONE && BonesToProcess[ParentBoneIndex];
				ParentBoneIndex = InSkeleton.GetParentIndex(ParentBoneIndex))
			{
				BoneIndex = ParentBoneIndex;
			}

			// Merge the bone
			const int32 NewBoneIndex = MergeBoneToSkeleton(BoneIndex, InSkeleton, InOutSkeleton);
			OutBoneMap[BoneIndex] = NewBoneIndex;

			// Marked it as processed
			BonesToProcess[BoneIndex] = false;
		}

		UE_LOGF(LogChaos, VeryVerbose, "-------- Outfit Reference Skeleton Merging --------");
		LogHierarchy(InOutSkeleton.GetRefBoneInfo());
		UE_LOGF(LogChaos, VeryVerbose, "---------------------------------------------------");
		for (int32 Index = 0; Index < OutBoneMap.Num(); ++Index)
		{
			UE_LOGF(LogChaos, VeryVerbose, "%d -> %d", Index, OutBoneMap[Index]);
		}
		UE_LOGF(LogChaos, VeryVerbose, "---------------------------------------------------");
	}

	TArray<uint32> GetIndices(const FMultiSizeIndexContainer& MultiSizeIndexContainer, int32 VertexOffset)
	{
		TArray<uint32> IndexBuffer;
		if (MultiSizeIndexContainer.IsIndexBufferValid())
		{
			MultiSizeIndexContainer.GetIndexBuffer(IndexBuffer);
			if (VertexOffset)
			{
				for (uint32& Index : IndexBuffer)
				{
					Index += VertexOffset;
				}
			}
		}
		return IndexBuffer;
	}

	TArray<FVector3f> GetPositions(const FPositionVertexBuffer& PositionVertexBuffer)
	{
		const uint32 NumVertices = PositionVertexBuffer.GetNumVertices();
		TArray<FVector3f> Positions;
		Positions.SetNumUninitialized(NumVertices);
		for (uint32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
		{
			Positions[VertexIndex] = PositionVertexBuffer.VertexPosition(VertexIndex);
		}
		return Positions;
	}

	TArray<FVector4f> GetTangents(const FStaticMeshVertexBuffer& StaticMeshVertexBuffer, int Axis)
	{
		const uint32 NumVertices = StaticMeshVertexBuffer.GetNumVertices();
		TArray<FVector4f> Tangents;
		Tangents.SetNumUninitialized(NumVertices);
		for (uint32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
		{
			switch (Axis)
			{
			default: checkNoEntry();
			case 0: Tangents[VertexIndex] = StaticMeshVertexBuffer.VertexTangentX(VertexIndex); break;
			case 1: Tangents[VertexIndex] = StaticMeshVertexBuffer.VertexTangentY(VertexIndex); break;
			case 2: Tangents[VertexIndex] = StaticMeshVertexBuffer.VertexTangentZ(VertexIndex); break;
			}
		}
		return Tangents;
	}

	TArray<FVector2f> GetVertexUVs(const FStaticMeshVertexBuffer& StaticMeshVertexBuffer, const uint32 MaxTexCoords)
	{
		const uint32 NumVertices = StaticMeshVertexBuffer.GetNumVertices();
		const uint32 NumTexCoords = StaticMeshVertexBuffer.GetNumTexCoords();
		check(NumTexCoords <= MaxTexCoords);

		TArray<FVector2f> VertexUVs;
		VertexUVs.SetNumUninitialized(NumVertices * MaxTexCoords);

		for (uint32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
		{
			for (uint32 UVIndex = 0; UVIndex < NumTexCoords; ++UVIndex)
			{
				VertexUVs[VertexIndex * MaxTexCoords + UVIndex] = StaticMeshVertexBuffer.GetVertexUV(VertexIndex, UVIndex);
			}
			for (uint32 UVIndex = NumTexCoords; UVIndex < MaxTexCoords; ++UVIndex)
			{
				VertexUVs[VertexIndex * MaxTexCoords + UVIndex] = FVector2f::ZeroVector;
			}
		}
		return VertexUVs;
	}

	TArray<FSkinWeightInfo> GetSkinWeights(const FSkinWeightVertexBuffer& SkinWeightVertexBuffer, const bool bUse16BitBoneWeight, const TArray<int32>* const BoneMap)
	{
		checkf(SkinWeightVertexBuffer.Use16BitBoneWeight() || !bUse16BitBoneWeight, TEXT("Weights can only be read from 8bit to 8bit, or 8bit to 16bit, but not 16bit to 8bit."));
		const bool bRenormalizeTo16BitBoneWeight = bUse16BitBoneWeight && !SkinWeightVertexBuffer.Use16BitBoneWeight();
		const uint32 MaxBoneInfluences = SkinWeightVertexBuffer.GetMaxBoneInfluences();
		const uint32 NumVertices = SkinWeightVertexBuffer.GetNumVertices();
		TArray<FSkinWeightInfo> SkinWeights;
		SkinWeights.SetNumUninitialized(NumVertices);
		for (uint32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
		{
			FSkinWeightInfo& SkinWeight = SkinWeights[VertexIndex];
			if (!bRenormalizeTo16BitBoneWeight)
			{
				for (uint32 InfluenceIndex = 0; InfluenceIndex < MaxBoneInfluences; ++InfluenceIndex)
				{
					SkinWeight.InfluenceBones[InfluenceIndex] = SkinWeightVertexBuffer.GetBoneIndex(VertexIndex, InfluenceIndex);
					SkinWeight.InfluenceWeights[InfluenceIndex] = SkinWeightVertexBuffer.GetBoneWeight(VertexIndex, InfluenceIndex);
				}
			}
			else
			{
				uint16 TotalInfluenceWeight = 0;
				for (uint32 InfluenceIndex = 0; InfluenceIndex < MaxBoneInfluences; ++InfluenceIndex)
				{
					SkinWeight.InfluenceBones[InfluenceIndex] = SkinWeightVertexBuffer.GetBoneIndex(VertexIndex, InfluenceIndex) << 8;
					SkinWeight.InfluenceWeights[InfluenceIndex] = SkinWeightVertexBuffer.GetBoneWeight(VertexIndex, InfluenceIndex);
					TotalInfluenceWeight += SkinWeight.InfluenceWeights[InfluenceIndex];
				}
				SkinWeight.InfluenceWeights[0] += TNumericLimits<uint16>::Max() - TotalInfluenceWeight;
			}
			// Clear the extra influences in case MaxBoneInfluences is changed
			for (uint32 InfluenceIndex = MaxBoneInfluences; InfluenceIndex < MAX_TOTAL_INFLUENCES; ++InfluenceIndex)
			{
				SkinWeight.InfluenceBones[InfluenceIndex] = 0;
				SkinWeight.InfluenceWeights[InfluenceIndex] = 0;
			}
		}
		if (BoneMap)
		{
			for (FSkinWeightInfo& SkinWeight : SkinWeights)
			{
				for (uint32 InfluenceIndex = 0; InfluenceIndex < MaxBoneInfluences; ++InfluenceIndex)
				{
					SkinWeight.InfluenceBones[InfluenceIndex] = (*BoneMap)[SkinWeight.InfluenceBones[InfluenceIndex]];
				}
			}
		}
		return SkinWeights;
	}

	TArray<FColor> GetVertexColors(const FStaticMeshVertexBuffers& StaticMeshVertexBuffers)
	{
		const uint32 NumVertices = StaticMeshVertexBuffers.PositionVertexBuffer.GetNumVertices();
		const FColorVertexBuffer& ColorVertexBuffer = StaticMeshVertexBuffers.ColorVertexBuffer;
		TArray<FColor> VertexColors;
		VertexColors.SetNumUninitialized(NumVertices);
		const bool bHasVertexColors = ColorVertexBuffer.GetAllocatedSize() != 0;
		if (bHasVertexColors)
		{
			check(ColorVertexBuffer.GetNumVertices() == NumVertices);
			for (uint32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
			{
				VertexColors[VertexIndex] = ColorVertexBuffer.VertexColor(VertexIndex);
			}
		}
		else
		{
			for (FColor& VertexColor : VertexColors)
			{
				VertexColor = FColor::White;
			}
		}
		return VertexColors;
	}

	TArray<FMeshToMeshVertData> GetClothMappingData(const TArray<FSkelMeshRenderSection>& RenderSections, const TArray<FGuid>* AssetGuids)
	{
		TArray<FMeshToMeshVertData> MappingData;
		int32 NumMappingData = 0;
		for (const FSkelMeshRenderSection& RenderSection : RenderSections)
		{
			if (!AssetGuids ||
				!RenderSection.ClothingData.AssetGuid.IsValid() ||
				AssetGuids->Find(RenderSection.ClothingData.AssetGuid) != INDEX_NONE)
			{
				for (const TArray<FMeshToMeshVertData>& ClothMappingDataLOD : RenderSection.ClothMappingDataLODs)
				{
					NumMappingData += ClothMappingDataLOD.Num();
				}
			}
		}
		MappingData.Reserve(NumMappingData);
		for (const FSkelMeshRenderSection& RenderSection : RenderSections)
		{
			if (!AssetGuids ||
				!RenderSection.ClothingData.AssetGuid.IsValid() ||
				AssetGuids->Find(RenderSection.ClothingData.AssetGuid) != INDEX_NONE)
			{
				for (const TArray<FMeshToMeshVertData>& ClothMappingDataLOD : RenderSection.ClothMappingDataLODs)
				{
					MappingData.Append(ClothMappingDataLOD);
				}
			}
		}
		return MappingData;
	}

	TArray<FClothBufferIndexMapping> GetClothBufferIndexMappings(
		const FSkeletalMeshVertexClothBuffer& ClothVertexBuffer,
		const TArray<FSkelMeshRenderSection>& RenderSections,
		const int32 VertexOffset,
		const TArray<FGuid>* AssetGuids)
	{
		int32 NumRenderSections = 0;
		for (const FSkelMeshRenderSection& RenderSection : RenderSections)
		{
			if (!AssetGuids ||
				!RenderSection.ClothingData.AssetGuid.IsValid() ||
				AssetGuids->Find(RenderSection.ClothingData.AssetGuid) != INDEX_NONE)
			{
				++NumRenderSections;
			}
		}

		TArray<FClothBufferIndexMapping> ClothBufferIndexMappings;
		if (!ClothVertexBuffer.GetClothIndexMapping().Num())
		{
			// Add empty mapping as some of the LOD sections have clothing
			ClothBufferIndexMappings.SetNumZeroed(NumRenderSections);  // FClothBufferIndexMapping has no default constructor
		}
		else
		{
			// Update the mappings index/offset
			for (int32 SectionIndex = 0; SectionIndex < RenderSections.Num(); ++SectionIndex)
			{
				const FSkelMeshRenderSection& RenderSection = RenderSections[SectionIndex];
				if (!AssetGuids ||
					!RenderSection.ClothingData.AssetGuid.IsValid() ||
					AssetGuids->Find(RenderSection.ClothingData.AssetGuid) != INDEX_NONE)
				{
					FClothBufferIndexMapping& ClothBufferIndexMapping = ClothBufferIndexMappings.Add_GetRef(ClothVertexBuffer.GetClothIndexMapping()[SectionIndex]);
					ClothBufferIndexMapping.BaseVertexIndex += VertexOffset;
					ClothBufferIndexMapping.MappingOffset += VertexOffset;
					// LODBiasStride stays the same since the number of mapping for this section hasn't changed
				}
			}
		}
		return ClothBufferIndexMappings;
	}

	void MergeBones(const FReferenceSkeleton& ReferenceSkeleton, const TArray<int32>& BoneMap, const TArray<FBoneIndexType>& BoneIndices, TArray<FBoneIndexType>& OutBoneIndices)
	{
		OutBoneIndices.Reserve(OutBoneIndices.Num() + BoneIndices.Num());
		for (const uint16 BoneIndex : BoneIndices)
		{
			OutBoneIndices.AddUnique(BoneMap[BoneIndex]);
		}
		ReferenceSkeleton.EnsureParentsExistAndSort(OutBoneIndices);
		OutBoneIndices.Shrink();
	}
}  // namespace UE::Chaos::OutfitAsset