// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosOutfitAsset/OutfitCollection.h"

namespace UE::Chaos::OutfitAsset
{
	/**
	 * Outfit collection reserved group and attribute names.
	 */
	namespace OutfitCollection
	{
		namespace Group
		{
			const FName Outfits(TEXT("OutfitCollection::Outfits"));
			const FName Pieces(TEXT("OutfitCollection::Pieces"));
			const FName BodySizes(TEXT("OutfitCollection::BodySizes"));
			const FName BodyParts(TEXT("OutfitCollection::BodyParts"));
			const FName Measurements(TEXT("OutfitCollection::Measurements"));
		}
		namespace Attribute
		{
			// Outfits group
			namespace Outfits
			{
				const FName Guid(TEXT("OutfitGuid"));  // FGuid, cannot simply be called Guid or it would get stripped from the ManagedArrayCollection serialization
				const FName BodySize(TEXT("BodySize"));  // int32 -> BodySizes
				const FName Name(TEXT("Name"));  // FString
				const FName PiecesStart(TEXT("PiecesStart"));  // int32 -> OutfitPieces
				const FName PiecesCount(TEXT("PiecesCount"));  // int32
			}
			// Pieces group
			namespace Pieces
			{
				const FName Guid(TEXT("PieceGuid"));  // FGuid, cannot simply be called Guid or it would get stripped from the ManagedArrayCollection serialization
				const FName Name(TEXT("Name"));  // FString
			}
			// BodySizes group
			namespace BodySizes
			{
				const FName Name(TEXT("Name"));  // FString
				const FName BodyPartsStart(TEXT("BodyPartsStart"));  // int32 -> BodyParts
				const FName BodyPartsCount(TEXT("BodyPartsCount"));  // int32
				const FName MeasurementsStart(TEXT("MeasurementsStart"));  // int32 -> Measurements
				const FName MeasurementsCount(TEXT("MeasurementsCount"));  // int32
			}
			// BodyParts group
			namespace BodyParts
			{
				const FName SkeletalMeshPath(TEXT("SkeletalMeshPath"));  // FSoftObjectPath
				UE_DEPRECATED(5.7, "Use SkeletalMeshPath instead.")
				const FName SkeletalMesh(TEXT("SkeletalMesh"));  // FString
				const FName RBFInterpolationSampleIndices(TEXT("RBFInterpolationSampleIndices"));  // TArray<int32>
				const FName RBFInterpolationSampleRestPositions(TEXT("RBFInterpolationSampleRestPositions"));  // TArray<FVector3f>
				const FName RBFInterpolationWeights(TEXT("RBFInterpolationWeights"));  // // TArray<float>
			}
			// Measurements group
			namespace Measurements
			{
				const FName Name(TEXT("Name"));  // FString
			}
		}
	}  // namespace OutfitCollection

	const FName DefaultBodySize(TEXT("Default"));
}  // namespace UE::Chaos::OutfitAsset
