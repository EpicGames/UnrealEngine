// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosOutfitAsset/SizedOutfitSource.h"
#include "ChaosOutfitAsset/OutfitCollection.h"
#include "ChaosClothAsset/ClothAssetBase.h"

FString FChaosSizedOutfitSource::GetBodySizeName() const
{
	FString BodySizeName = SizeName;
	if (BodySizeName.IsEmpty())
	{
		// Use the skeletal mesh name if the size name is empty
		if (const TObjectPtr<const USkeletalMesh>* const SkeletalMesh = SourceBodyParts.FindByPredicate(
			[](const TObjectPtr<const USkeletalMesh>& SourceBodyPart) -> bool
			{
				return !!SourceBodyPart;
			}))
		{
			BodySizeName = SkeletalMesh->GetName();
		}
		else if (SourceAsset && SourceAsset->GetNumClothSimulationModels())
		{
			// Return "Default" if the source asset is valid and no body parts have been provided
			BodySizeName = UE::Chaos::OutfitAsset::DefaultBodySize.ToString();
		}
	}
	return BodySizeName;
}
