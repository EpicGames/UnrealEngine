// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/AssetUserData.h"
#include "BodyUserData.generated.h"

/**
 * Asset user data attached to the merged body and face skeletal mesh.
 */
UCLASS(MinimalAPI)
class UChaosOutfitAssetBodyUserData : public UAssetUserData
{
	GENERATED_BODY()

public:
	static inline constexpr float InvalidMeasurement = 0.f;

	UPROPERTY(VisibleAnywhere, Category = "Measurements")
	TMap<FString, float> Measurements;
};
