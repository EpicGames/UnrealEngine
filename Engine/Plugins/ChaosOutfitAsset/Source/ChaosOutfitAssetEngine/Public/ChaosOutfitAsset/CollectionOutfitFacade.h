// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "ChaosOutfitAsset/OutfitCollection.h"

#define UE_API CHAOSOUTFITASSETENGINE_API

template<typename T> class TManagedArray;
class UChaosClothAssetBase;
class USkeletalMesh;
struct FManagedArrayCollection;

namespace UE::Chaos::OutfitAsset
{
	namespace Private
	{
		struct FOutfitCollection;
	}

	/**
	 * Cloth outfit collection facade.
	 * Const access (read only) version.
	 */
	class FCollectionOutfitConstFacade
	{
	public:
		struct FRBFInterpolationDataWrapper
		{
			TConstArrayView<TArray<int32>> SampleIndices;
			TConstArrayView<TArray<FVector3f>> SampleRestPositions;
			TConstArrayView<TArray<float>> InterpolationWeights;
		};

		UE_API explicit FCollectionOutfitConstFacade(const TSharedRef<const FManagedArrayCollection>& InManagedArrayCollection);
		UE_API explicit FCollectionOutfitConstFacade(const FManagedArrayCollection& InManagedArrayCollection);

		FCollectionOutfitConstFacade() = delete;
		FCollectionOutfitConstFacade(const FCollectionOutfitConstFacade&) = delete;
		FCollectionOutfitConstFacade& operator=(const FCollectionOutfitConstFacade&) = delete;
		FCollectionOutfitConstFacade(FCollectionOutfitConstFacade&&) = default;
		FCollectionOutfitConstFacade& operator=(FCollectionOutfitConstFacade&&) = default;
		UE_API virtual ~FCollectionOutfitConstFacade();

		/** Return whether the facade is defined on the collection. */
		UE_API bool IsValid() const;

		/** Return the sized outfits GUID, one per sized outfit which can contain different sized outfits. */
		UE_API TArray<FGuid> GetOutfitGuids() const;

		/** Return whether this outfit has at least one valid body size. Definition of valid depends on the passed arguments. */
		UE_API bool HasValidBodySize(bool bBodyPartMustExist = false, bool bMeasurementsMustExist = false, bool bInterpolationDataMustExist = false) const;

		/** Return all unique body part skeletal meshes used by this outfit. */
		UE_API TArray<FSoftObjectPath> GetOutfitBodyPartsSkeletalMeshPaths() const;
		UE_DEPRECATED(5.7, "Use GetOutfitBodyPartsSkeletalMeshPaths instead.")
		UE_API TArray<FString> GetOutfitBodyPartsSkeletalMeshes() const;

		/** Return the body size available for the specified outfit GUID. */
		UE_API TArray<int32> GetOutfitBodySizes(const FGuid& Guid) const;

		/** Return the name of the original asset making up the specified outfit body size. */
		UE_API const FString& GetOutfitName(const FGuid& Guid, const int32 BodySize) const;

		/** Return all outfit pieces asset GUIDs. */
		UE_API TConstArrayView<FGuid> GetOutfitPiecesGuids() const;

		/** Return the outfit pieces making up the specified outfit body size. */
		UE_API TMap<FGuid, FString> GetOutfitPieces(const FGuid& Guid, const int32 BodySize) const;

		/** Return the number of body sizes. */
		UE_API int32 GetNumBodySizes() const;

		/** Return whether this body size already exist. */
		bool HasBodySize(const FString& Name) const
		{
			return FindBodySize(Name) != INDEX_NONE;
		}

		/** Return the body size index for the specified name, or INDEX_NONE if it doesn't exist in this outfit. */
		UE_API int32 FindBodySize(const FString& Name) const;

		/** Return the closest size to the specified measurements, or INDEX_NONE if this outfit has no sizes or the measurements are incomplete. */
		UE_API int32 FindClosestBodySize(const TMap<FString, float>& Measurements) const;

		/** Return the closest size to the specified body, or INDEX_NONE if this outfit has no sizes or the body has no measurement data. */
		UE_API int32 FindClosestBodySize(const USkeletalMesh& BodyPart) const;

		/** Return the specified body size name which is unique. */
		UE_API const FString& GetBodySizeName(const int32 BodySize) const;

		/** Return the body part skeletal meshes path name for the specified body size. */
		UE_API TConstArrayView<FSoftObjectPath> GetBodySizeBodyPartsSkeletalMeshPaths(const int32 BodySize) const;
		UE_DEPRECATED(5.7, "Use GetBodySizeBodyPartsSkeletalMeshPaths instead.")
		TConstArrayView<FString> GetBodySizeBodyPartsSkeletalMeshes(const int32 BodySize) const
		{
			return TConstArrayView<FString>();
		}

		/** Return the body size body part offset (to index into the global body part array). Returns INDEX_NONE if BodySize is invalid.*/
		UE_API int32 GetBodySizeBodyPartOffset(const int32 BodySize) const;

		/** Return the body size body part count. Returns 0 if BodySize is invalid.*/
		UE_API int32 GetBodySizeBodyPartCount(const int32 BodySize) const;

		/** Return  for the specified body size. */
		UE_API FRBFInterpolationDataWrapper GetBodySizeInterpolationData(const int32 BodySize) const;

		/** Return the body measurements stored for this specified body size. */
		UE_API TMap<FString, float> GetBodySizeMeasurements(const int32 BodySize) const;


	protected:
		const Private::FOutfitCollection* GetOutfitCollection() const { return OutfitCollection.Get(); }

		TUniquePtr<const Private::FOutfitCollection> OutfitCollection;
		TSharedPtr<const FManagedArrayCollection> ManagedArrayCollection;  // Only used to keep the shared ref alive
	};

	/**
	 * Cloth outfit collection facade.
	 * Non-const access (read/write) version.
	 */
	class FCollectionOutfitFacade final : public FCollectionOutfitConstFacade
	{
	public:
		UE_API explicit FCollectionOutfitFacade(const TSharedRef<FManagedArrayCollection>& InManagedArrayCollection);
		UE_API explicit FCollectionOutfitFacade(FManagedArrayCollection& InManagedArrayCollection);
	
		FCollectionOutfitFacade() = delete;
		FCollectionOutfitFacade(const FCollectionOutfitFacade&) = delete;
		FCollectionOutfitFacade& operator=(const FCollectionOutfitFacade&) = delete;
		FCollectionOutfitFacade(FCollectionOutfitFacade&&) = default;
		FCollectionOutfitFacade& operator=(FCollectionOutfitFacade&&) = default;
		UE_API virtual ~FCollectionOutfitFacade() override;

		/** Do post-serialization fixups */
		UE_API void PostSerialize(const FArchive& Ar);

		/** Add the Selection attributes to the underlying collection. */
		UE_API void DefineSchema();

		/** Remove all outfit and size information from this collection. */
		UE_API void Reset();

		/**
		 * Add a new body size and returns its index.
		 * Any pre-existing sizes data will be replaced.
		 */
		UE_API int32 AddBodySize(
			const FString& Name,
			const TConstArrayView<FSoftObjectPath> BodyPartsSkeletalMeshes,
			const TMap<FString, float>& Measurements,
			const FRBFInterpolationDataWrapper& InterpolationData);
		UE_DEPRECATED(5.7, "Use version with FSoftObjectPath BodyPartsSkeletalMeshes")
		UE_API int32 AddBodySize(
			const FString& Name,
			const TConstArrayView<FString> BodyPartsSkeletalMeshes,
			const TMap<FString, float>& Measurements,
			const FRBFInterpolationDataWrapper& InterpolationData);

		/** Find or add a new body size and returns its index. */
		int32 FindOrAddBodySize(const FString& Name)
		{
			const int32 BodySize = FindBodySize(Name);

			return BodySize != INDEX_NONE ? BodySize : AddBodySize(Name, TConstArrayView<FSoftObjectPath>(), {}, {});
		}

		/**
		 * Add a new sized outfit under the specified outfit GUID.
		 * Any pre-existing sizes data will be replaced.
		 */
		UE_API void AddOutfit(
			const FGuid& Guid,
			const int32 BodySize,
			const UChaosClothAssetBase& ClothAssetBase);

		/**
		 * Append an existing outfit facade to this collection.
		 * Any pre-existing sizes data will be replaced.
		 * @param Other The source outfit collection to get the data from.
		 * @param SelectedBodySize The body size of the source outfit to append, or INDEX_NONE to append all sizes.
		 */
		UE_API void Append(const FCollectionOutfitConstFacade& Other, int32 SelectedBodySize = INDEX_NONE);

		/**
		 * Append an existing outfit facade to this collection.
		 * Any pre-existing sizes data will be replaced.
		 */
		void Append(const FManagedArrayCollection& InManagedArrayCollection)
		{
			FCollectionOutfitConstFacade Other(InManagedArrayCollection);
			Append(Other);
		}

	private:
		void AddOutfit(
			const FGuid& Guid,
			const int32 BodySize,
			const FString& Name,
			const TMap<FGuid, FString>& Pieces);

		Private::FOutfitCollection* GetOutfitCollection() { return const_cast<Private::FOutfitCollection*>(OutfitCollection.Get()); }
	};
}

#undef UE_API
