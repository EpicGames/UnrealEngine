// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Templates/SharedPointerFwd.h"
#include "UObject/Object.h"
#include "ReferenceSkeleton.h"

#include "Outfit.generated.h"

#define UE_API CHAOSOUTFITASSETENGINE_API

class FSkeletalMeshRenderData;
class UChaosClothAssetBase;
class UPhysicsAsset;
class USkeletalMesh;
struct FChaosClothSimulationModel;
struct FChaosSizedOutfitSource;
struct FManagedArrayCollection;
struct FSkeletalMaterial;

USTRUCT()
struct FChaosOutfitPiece final
{
	GENERATED_BODY()

	UPROPERTY()
	FName Name;

	UPROPERTY()
	FGuid AssetGuid;

	UPROPERTY()
	TObjectPtr<const UPhysicsAsset> PhysicsAsset;

	TSharedRef<FChaosClothSimulationModel> ClothSimulationModel;

	TArray<TSharedRef<const FManagedArrayCollection>> Collections;

	UE_API FChaosOutfitPiece();
	UE_API ~FChaosOutfitPiece();
	UE_API FChaosOutfitPiece(
		FName InName, 
		FGuid InAssetGuid, 
		const UPhysicsAsset* InPhysicsAsset, 
		const FChaosClothSimulationModel& InClothSimulationModel, 
		const TArray<TSharedRef<const FManagedArrayCollection>>& InCollections);

	UE_API FChaosOutfitPiece(const UChaosClothAssetBase& ClothAssetBase, int32 ModelIndex);
	FChaosOutfitPiece(const TArray<FChaosOutfitPiece>& Pieces, int32 ModelIndex) : FChaosOutfitPiece(Pieces[ModelIndex]) {}

	UE_API FChaosOutfitPiece(const FChaosOutfitPiece& Other);

	UE_API FChaosOutfitPiece(FChaosOutfitPiece&& Other);

	UE_API FChaosOutfitPiece& operator=(const FChaosOutfitPiece& Other);
	UE_API FChaosOutfitPiece& operator=(FChaosOutfitPiece&& Other);

	UE_API bool Serialize(FArchive& Ar);

private:
	friend UChaosOutfit;

	void DeepCopyCollections(const TArray<TSharedRef<const FManagedArrayCollection>>& Other);

	void RemapBoneIndices(const TArray<int32>& BoneMap);
};

template<>
struct TStructOpsTypeTraits<FChaosOutfitPiece> : public TStructOpsTypeTraitsBase2<FChaosOutfitPiece>
{
	enum
	{
		WithSerializer = true,
	};
};

/**
 * Outfit class to handle the assembly of an outfit asset.
 */
UCLASS(MinimalAPI, BlueprintType)
class UChaosOutfit final : public UObject
{
	GENERATED_BODY()

public:
	UE_API UChaosOutfit(const FObjectInitializer& ObjectInitializer);
	UE_API UChaosOutfit(FVTableHelper& Helper);  // This is declared so we can use TUniquePtr<FClothSimulationModel> with just a forward declare of that class
	UE_API virtual ~UChaosOutfit() override;

	UE_API void Append(const UChaosOutfit& Other, const FString& BodySizeNameFilter = {});

	UE_API void Add(const UChaosClothAssetBase& ClothAssetBase);
	UE_API void Add(const FChaosSizedOutfitSource& SizedOutfitSource, const FGuid& OutfitGuid = FGuid::NewGuid());

	UE_API void CopyTo(
		TArray<FChaosOutfitPiece>& OutPieces,
		FReferenceSkeleton& OutReferenceSkeleton,
		TUniquePtr<FSkeletalMeshRenderData>& OutSkeletalMeshRenderData,
		TArray<FSkeletalMaterial>& OutMaterials,
		FManagedArrayCollection& OutOutfitCollection) const;

	const TArray<FChaosOutfitPiece>& GetPieces() const
	{
		return Pieces;
	}

	TArrayView<FChaosOutfitPiece> GetPieces()
	{
		return Pieces;
	}

	const TArray<FSkeletalMaterial>& GetMaterials() const
	{
		return Materials;
	}

	TArray<FSkeletalMaterial>& GetMaterials()
	{
		return Materials;
	}

	const FManagedArrayCollection& GetOutfitCollection() const
	{
		return OutfitCollection;
	}
	FManagedArrayCollection& GetOutfitCollection()
	{
		return OutfitCollection;
	}

	/** Return the number of LODs (max LOD contained in any piece). */
	UE_API int32 GetNumLods() const;

	/** Return the outfit pieces cloth collections for all LODs. */
	UE_API TArray<TSharedRef<const FManagedArrayCollection>> GetClothCollections(int32 LodIndex) const;

	const FReferenceSkeleton& GetReferenceSkeleton() const
	{
		return ReferenceSkeleton;
	}

	UE_API bool HasBodySize(const FString& SizeName) const;

	//~ Begin UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject interface

	struct FLODRenderData;
	struct FRenderData;

	template<typename RenderDataType UE_REQUIRES(std::is_same_v<RenderDataType, FSkeletalMeshRenderData> || std::is_same_v<RenderDataType, UChaosOutfit::FRenderData>)>
	static void Init(
		TArray<FChaosOutfitPiece>& OutPieces,
		FReferenceSkeleton& OutReferenceSkeleton,
		TUniquePtr<RenderDataType>& OutSkeletalMeshRenderData,
		TArray<FSkeletalMaterial>& OutMaterials,
		FManagedArrayCollection& OutOutfitCollection);
private:
	template<typename InLODRenderDataType, typename OutLODRenderDataType>
	static void MergeLODRenderDatas(
		const InLODRenderDataType& LODRenderData,
		const TArray<FGuid>& AssetGuids,
		const int32 MaterialOffset,
		const FReferenceSkeleton& ReferenceSkeleton,
		const TArray<int32>& BoneMap,
		OutLODRenderDataType& OutLODRenderData);

	template<typename InRenderDataType, typename OutRenderDataType, typename PiecesType>
	static void Merge(
		const FReferenceSkeleton& InReferenceSkeleton,
		const InRenderDataType* const InRenderData,
		const TArray<FSkeletalMaterial>& InMaterials,
		const FManagedArrayCollection& InOutfitCollection,
		const PiecesType& Pieces,
		const int32 NumPieces,
		const FString& BodySizeNameFilter,
		TArray<FChaosOutfitPiece>& OutPieces,
		FReferenceSkeleton& OutReferenceSkeleton,
		OutRenderDataType* const OutRenderData,
		TArray<FSkeletalMaterial>& OutMaterials,
		FManagedArrayCollection& OutOutfitCollection);

	UPROPERTY()
	TArray<FChaosOutfitPiece> Pieces;

	UPROPERTY()
	TArray<FSkeletalMaterial> Materials;

	UPROPERTY()
	FManagedArrayCollection OutfitCollection;

	FReferenceSkeleton ReferenceSkeleton;

	TUniquePtr<FRenderData> RenderData;
};

#undef UE_API
