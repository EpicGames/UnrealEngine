// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosClothAsset/ClothAssetBase.h"
#include "ChaosOutfitAsset/Outfit.h"
#include "OutfitAsset.generated.h"

#define UE_API CHAOSOUTFITASSETENGINE_API

struct FChaosOutfitPiece;
namespace UE::Dataflow { struct IContextAssetStoreInterface; }

/**
 * Outfit asset for character clothing and simulation.
 */
UCLASS(MinimalAPI, HideCategories = Object, BlueprintType, PrioritizeCategories = ("Dataflow"))
class UChaosOutfitAsset final : public UChaosClothAssetBase
{
	GENERATED_BODY()
public:
	UE_API UChaosOutfitAsset(const FObjectInitializer& ObjectInitializer);
	UE_API UChaosOutfitAsset(FVTableHelper& Helper);  // This is declared so we can use TUniquePtr<FClothSimulationModel> with just a forward declare of that class
	UE_API virtual ~UChaosOutfitAsset() override;

	UE_API void Build(const TObjectPtr<const UChaosOutfit> InOutfit, UE::Dataflow::IContextAssetStoreInterface* ContextAssetStore = nullptr, bool bDeferResourceInit = false);

	//~ Begin UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;
	//~ End UObject interface

	const FManagedArrayCollection& GetOutfitCollection() const
	{
		return OutfitCollection;
	}

#if WITH_EDITORONLY_DATA
	TObjectPtr<const UChaosOutfit> GetOutfit() const
	{
		return Outfit;
	}
#endif

private:
	//~ Begin UChaosClothAssetBase interface
	UE_API virtual bool HasValidClothSimulationModels() const override;
	virtual int32 GetNumClothSimulationModels() const override
	{
		return Pieces.Num();
	}
	UE_API virtual FName GetClothSimulationModelName(int32 ModelIndex) const override;
	UE_API virtual TSharedPtr<const FChaosClothSimulationModel> GetClothSimulationModel(int32 ModelIndex) const override;
	UE_API virtual const TArray<TSharedRef<const FManagedArrayCollection>>& GetCollections(int32 ModelIndex) const override;
	UE_API virtual const UPhysicsAsset* GetPhysicsAssetForModel(int32 ModelIndex) const override;
	UE_API virtual FGuid GetAssetGuid(int32 ModelIndex) const override;
#if WITH_EDITOR
	virtual UChaosClothAssetBase* CreatePreviewAssetCopy(UObject* Outer, EObjectFlags Flags) const override
	{
		constexpr bool bFilterToSingleSize = true;
		constexpr bool bBuildSimModelRenderData = true;
		return CreatePreviewAssetCopyImpl(Outer, Flags, bFilterToSingleSize, bBuildSimModelRenderData);
	}

#endif
	//~ End UChaosClothAssetBase interface

	//~ Begin USkinnedAsset interface
	virtual UPhysicsAsset* GetPhysicsAsset() const override
	{
		return nullptr;  // There isn't a single Physics Asset anymore, this could return the first one but that wouldn't be accurate
	}
	virtual USkeleton* GetSkeleton() override
	{
		return nullptr;  // Note: The USkeleton isn't a reliable source of reference skeleton
	}
	virtual const USkeleton* GetSkeleton() const override
	{
		return nullptr;
	}
	virtual void SetSkeleton(USkeleton* InSkeleton) override {}
#if WITH_EDITOR
	UE_API virtual FString BuildDerivedDataKey(const ITargetPlatform* TargetPlatform) override;
	virtual void BuildLODModel(FSkeletalMeshRenderData& RenderData, const ITargetPlatform* TargetPlatform, int32 LODIndex) override;
	virtual const TCHAR* GetDerivedDataPrefix() const override
	{
		return TEXT("CHAOSOUTFIT");
	}
	UE_API virtual const TCHAR* GetDerivedDataVersion() const override;
	virtual void BeginPostLoadAssetImpl(FSkinnedAssetPostLoadContext& Context) override;
#endif
	//~ End USkinnedAsset interface

	/** Return the number of LODs derived from Pieces (at least 1). */
	int32 GetNumLODsFromPieces() const;

#if WITH_EDITOR
	struct FBuilder;  // Defined in OutfitAssetBuilder.h

	/**
	 * Create a preview copy with outfit-specific options.
	 * @param Outer The outer object for the preview copy.
	 * @param Flags Object flags for the preview copy.
	 * @param bFilterToSingleSize If true, multi-size outfits are filtered to the first body size.
	 * @param bBuildSimModelRenderData If true and the asset has no render data, creates render data
	 *   from the simulation model vertex positions, normals, and bone weights for preview visualization.
	 * @return A transient asset copy, or nullptr if this asset can be used directly.
	 */
	UE_API UChaosClothAssetBase* CreatePreviewAssetCopyImpl(
		UObject* Outer,
		EObjectFlags Flags,
		bool bFilterToSingleSize,
		bool bBuildSimModelRenderData) const;
#endif

	UPROPERTY()
	TArray<FChaosOutfitPiece> Pieces;

	UPROPERTY()
	TArray<TObjectPtr<USkeletalMesh>> Bodies;  // Only contains dependencies, populated from the outfit collection

	UPROPERTY()
	FManagedArrayCollection OutfitCollection;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Instanced)
	TObjectPtr<UChaosOutfit> Outfit;  // Outfit source model used for generating this outfit asset
#endif
};

#undef UE_API
