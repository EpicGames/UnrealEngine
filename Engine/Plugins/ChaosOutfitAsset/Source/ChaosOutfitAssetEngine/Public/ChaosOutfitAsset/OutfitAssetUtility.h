// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoneIndices.h"

struct FReferenceSkeleton;
struct FMeshBoneInfo;
struct FSkinWeightInfo;
struct FSkelMeshRenderSection;
struct FMeshToMeshVertData;
struct FClothBufferIndexMapping;
class FMultiSizeIndexContainer;
class FPositionVertexBuffer;
class FStaticMeshVertexBuffer;
struct FStaticMeshVertexBuffers;
class FSkinWeightVertexBuffer;
class FSkeletalMeshVertexClothBuffer;

namespace UE::Chaos::OutfitAsset
{
	// Merge bone to a new skeleton and return its new index in the new skeleton
	CHAOSOUTFITASSETENGINE_API int32 MergeBoneToSkeleton(const int32 BoneIndex, const FReferenceSkeleton& InSkeleton, FReferenceSkeleton& InOutSkeleton);

	CHAOSOUTFITASSETENGINE_API TArray<int32> GetChildren(const TArray<FMeshBoneInfo>& MeshBoneInfos, const int32 ParentIndex);

	CHAOSOUTFITASSETENGINE_API void LogHierarchy(const TArray<FMeshBoneInfo>& MeshBoneInfos, const int32 ParentIndex = INDEX_NONE, int32 Indent = 0);

	CHAOSOUTFITASSETENGINE_API void MergeSkeletons(const FReferenceSkeleton& InSkeleton, FReferenceSkeleton& InOutSkeleton, TArray<int32>& OutBoneMap);

	CHAOSOUTFITASSETENGINE_API TArray<uint32> GetIndices(const FMultiSizeIndexContainer& MultiSizeIndexContainer, int32 VertexOffset = 0);

	CHAOSOUTFITASSETENGINE_API TArray<FVector3f> GetPositions(const FPositionVertexBuffer& PositionVertexBuffer);

	CHAOSOUTFITASSETENGINE_API TArray<FVector4f> GetTangents(const FStaticMeshVertexBuffer& StaticMeshVertexBuffer, int Axis);

	CHAOSOUTFITASSETENGINE_API TArray<FVector2f> GetVertexUVs(const FStaticMeshVertexBuffer& StaticMeshVertexBuffer, const uint32 MaxTexCoords);

	CHAOSOUTFITASSETENGINE_API TArray<FSkinWeightInfo> GetSkinWeights(const FSkinWeightVertexBuffer& SkinWeightVertexBuffer, const bool bUse16BitBoneWeight, const TArray<int32>* const BoneMap = nullptr);

	CHAOSOUTFITASSETENGINE_API TArray<FColor> GetVertexColors(const FStaticMeshVertexBuffers& StaticMeshVertexBuffers);

	CHAOSOUTFITASSETENGINE_API TArray<FMeshToMeshVertData> GetClothMappingData(const TArray<FSkelMeshRenderSection>& RenderSections, const TArray<FGuid>* AssetGuids = nullptr);

	CHAOSOUTFITASSETENGINE_API TArray<FClothBufferIndexMapping> GetClothBufferIndexMappings(
		const FSkeletalMeshVertexClothBuffer& ClothVertexBuffer,
		const TArray<FSkelMeshRenderSection>& RenderSections,
		const int32 VertexOffset = 0,
		const TArray<FGuid>* AssetGuids = nullptr);

	CHAOSOUTFITASSETENGINE_API void MergeBones(const FReferenceSkeleton& ReferenceSkeleton, const TArray<int32>& BoneMap, const TArray<FBoneIndexType>& BoneIndices, TArray<FBoneIndexType>& OutBoneIndices);

}  // namespace UE::Chaos::OutfitAsset