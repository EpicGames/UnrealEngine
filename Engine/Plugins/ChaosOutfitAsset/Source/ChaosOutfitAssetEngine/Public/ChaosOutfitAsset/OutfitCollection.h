// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

#define UE_API CHAOSOUTFITASSETENGINE_API

namespace UE::Chaos::OutfitAsset
{
	/**
	 * Outfit collection reserved group and attribute names.
	 */
	namespace OutfitCollection
	{
		namespace Group
		{
			UE_API extern const FName Outfits;
			UE_API extern const FName Pieces;
			UE_API extern const FName BodySizes;
			UE_API extern const FName BodyParts;
			UE_API extern const FName Measurements;
		}
		namespace Attribute
		{
			namespace Outfits
			{
				UE_API extern const FName Guid;  // FGuid
				UE_API extern const FName BodySize;  // int32 -> BodySizes
				UE_API extern const FName Name;  // FString
				UE_API extern const FName PiecesStart;  // int32 -> Pieces
				UE_API extern const FName PiecesCount;  // int32
			}
			namespace Pieces
			{
				UE_API extern const FName Guid;  // FGuid
				UE_API extern const FName Name;  // FString
			}
			namespace BodySizes
			{
				UE_API extern const FName Name;  // FString
				UE_API extern const FName BodyPartsStart;  // int32 -> BodyParts
				UE_API extern const FName BodyPartsCount;  // int32
			}
			namespace BodyParts
			{
				UE_API extern const FName SkeletalMeshPath;  // FSoftObjectPath
				UE_DEPRECATED(5.7, "Use SkeletalMeshPath instead.")
				UE_API extern const FName SkeletalMesh;  // FString
				UE_API extern const FName RBFInterpolationSampleIndices;  // TArray<int32>
				UE_API extern const FName RBFInterpolationSampleRestPositions;  // TArray<FVector3f>
				UE_API extern const FName RBFInterpolationWeights;  // TArray<float>
			}
			namespace Measurements
			{
				UE_API extern const FName Name;  // FString
			}
		}
	}  // namespace OutfitCollection

	/** Default body size name, when an outfit is created without any specific body size. */
	CHAOSOUTFITASSETENGINE_API extern const FName DefaultBodySize;

	/** Default number of RBF Interpolation points used for RBF Resizing */
	inline constexpr int32 DefaultNumRBFInterpolationPoints = 1500;
}  // namespace UE::Chaos::OutfitAsset

#undef UE_API
