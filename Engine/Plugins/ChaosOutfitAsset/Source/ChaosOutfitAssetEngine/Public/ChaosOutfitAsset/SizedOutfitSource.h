// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Containers/Array.h"
#include "UObject/ObjectPtr.h"
#include "ChaosOutfitAsset/OutfitCollection.h"
#include "SizedOutfitSource.generated.h"

class UChaosClothAssetBase;
class USkeletalMesh;

/**
 * Input structure for setting up a single sized outfit.
 */
USTRUCT(BlueprintType)  // BlueprintType to allow this structure as a variable type.
struct FChaosSizedOutfitSource final
{
	GENERATED_USTRUCT_BODY()

	/**
	 * The cloth or outfit asset to assign for this body size.
	 * The asset must be the exact same garment representation for each specified sizes.
	 * If multiple garments in multiple sizes are needed, they will have to be composed in another Outfit asset.
	 */
	UPROPERTY(EditAnywhere, Category = "OutfitSourceAsset")
	TObjectPtr<const UChaosClothAssetBase> SourceAsset;

	/**
	 * The unique name of this body size.
	 * The name of the first valid body part skeletal mesh will be used if empty.
	 */
	UPROPERTY(EditAnywhere, Category = "OutfitBodySize")
	FString SizeName;

	/**
	 * The list of body part skeletal meshes making up the source body for this size name.
	 * Usually one single MetaHuman merged body+head skeletal mesh.
	 */
	UPROPERTY(EditAnywhere, Category = "OutfitBodySize", Meta = (NoElementDuplicate))
	TArray<TObjectPtr<const USkeletalMesh>> SourceBodyParts = { nullptr };

	/**
	 * The number of interpolation points used in the resizing algorithm. These points are distributed evenly across the entire body.
	 * Increasing this number increases the quality of the resizing operation, but at additional cost, including the initial operation of 
	 *  generating your resizable Outfit, and the size of your Outfit on disk.
	 * If you find you cannot generate acceptable resizing results by increasing this number, we recommend adding a new Size.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "OutfitBodySize", Meta = (ClampMin = 0, ClampMax = 5000))
	int32 NumResizingInterpolationPoints = UE::Chaos::OutfitAsset::DefaultNumRBFInterpolationPoints;

	/**
	 * Get a usable name for the body size.
	 * Return SizeName unless empty, in which case return the name of the first valid body part.
	 * If both SizeName and the SourceBodyParts are empty, but a valid SourceAsset is provided,
	 * it will return "Default" as the body size name.
	 * Otherwise if none of these are valid, an empty string is returned.
	 */
	CHAOSOUTFITASSETENGINE_API FString GetBodySizeName() const;
};
